{
	"FileVersion": 3,

	"FriendlyName": "Online Services EOS (Game Services)",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Online Services implementation for EOS Game services only.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "OnlineServicesEOSGS",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux",
				"LinuxArm64",
				"Android"
			]
		}
	],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemEOSGS",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServicesEpicCommon",
			"Enabled": true
		},
		{
			"Name": "OnlineServices",
			"Enabled": true
		},
		{
			"Name": "EOSShared",
			"Enabled": true
		},
		{
			"Name": "SocketSubsystemEOS",
			"Enabled": true
		}
	]
}