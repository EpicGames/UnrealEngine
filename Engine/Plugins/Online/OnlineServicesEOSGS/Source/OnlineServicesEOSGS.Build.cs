// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineServicesEOSGS : ModuleRules
{
	public OnlineServicesEOSGS(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"EOSSDK",
				"OnlineServicesInterface",
				"OnlineServicesEpicCommon",
				"OnlineServicesCommon",
				"OnlineServicesCommonEngineUtils",
				"OnlineBase",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreOnline",
				"CoreUObject",
				"EOSShared",
				"Sockets"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"SocketSubsystemEOS"
				}
			);
		}

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PublicWeakFrameworks.Add("AuthenticationServices");
			PrivateDependencyModuleNames.Add("ApplicationCore");
		}
	}
}
