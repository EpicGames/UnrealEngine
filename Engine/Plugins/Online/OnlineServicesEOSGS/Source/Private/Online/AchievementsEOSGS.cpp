// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AchievementsEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AchievementsErrors.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_achievements.h"

namespace UE::Online {

	FAchievementsEOSGS::FAchievementsEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

void FAchievementsEOSGS::Initialize()
{
	Super::Initialize();

	AchievementsHandle = EOS_Platform_GetAchievementsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(AchievementsHandle);

	OnAchievementsUnlocked = EOS_RegisterComponentEventHandler(
		this,
		AchievementsHandle,
		2,
		&EOS_Achievements_AddNotifyAchievementsUnlockedV2,
		&EOS_Achievements_RemoveNotifyAchievementsUnlocked,
		&FAchievementsEOSGS::HandleAchievementsUnlocked);
	UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_ADDNOTIFYACHIEVEMENTSUNLOCKEDV2_API_LATEST, 2);
}

void FAchievementsEOSGS::Shutdown()
{
	Super::Shutdown();

	OnAchievementsUnlocked = nullptr;
}

TOnlineAsyncOpHandle<FQueryAchievementDefinitions> FAchievementsEOSGS::QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementDefinitions> Op = GetJoinableOp<FQueryAchievementDefinitions>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& InAsyncOp)
		{
			const FQueryAchievementDefinitions::Params& Params = InAsyncOp.GetParams();

			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
			else if (AchievementDefinitions.IsSet())
			{
				InAsyncOp.SetResult({});
			}
		})
		.Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& InAsyncOp, TPromise<const EOS_Achievements_OnQueryDefinitionsCompleteCallbackInfo*>&& Promise)
		{
			const FQueryAchievementDefinitions::Params& Params = InAsyncOp.GetParams();

			EOS_Achievements_QueryDefinitionsOptions Options = {};
			Options.ApiVersion = 3;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_QUERYDEFINITIONS_API_LATEST, 3);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

			EOS_Async(EOS_Achievements_QueryDefinitions, AchievementsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& InAsyncOp, const EOS_Achievements_OnQueryDefinitionsCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Achievements_QueryDefinitions failed with result=[%ls]", *LexToString(Data->ResultCode));

				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			EOS_Achievements_GetAchievementDefinitionCountOptions GetCountOptions = {};
			GetCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_GETACHIEVEMENTDEFINITIONCOUNT_API_LATEST, 1);

			const uint32_t AchievementCount = EOS_Achievements_GetAchievementDefinitionCount(AchievementsHandle, &GetCountOptions);

			FAchievementDefinitionMap NewAchievementDefinitions;
			for (uint32_t AchievementIdx = 0; AchievementIdx < AchievementCount; AchievementIdx++)
			{
				EOS_Achievements_CopyAchievementDefinitionV2ByIndexOptions CopyOptions = {};
				CopyOptions.ApiVersion = 2;
				UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_COPYACHIEVEMENTDEFINITIONV2BYINDEX_API_LATEST, 2);
				CopyOptions.AchievementIndex = AchievementIdx;

				EOS_Achievements_DefinitionV2* EOSDefinition = nullptr;
				const EOS_EResult CopyResult = EOS_Achievements_CopyAchievementDefinitionV2ByIndex(AchievementsHandle, &CopyOptions, &EOSDefinition);
				if (CopyResult != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_Achievements_CopyAchievementDefinitionV2ByIndex failed with result=[%ls]", *LexToString(CopyResult));

					InAsyncOp.SetError(Errors::FromEOSResult(CopyResult));
					return;
				}

				const int ExpectedDefinitionApiVersion = 2;
				UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_DEFINITIONV2_API_LATEST, ExpectedDefinitionApiVersion);
				const bool bDefsApiVersionOk = EOSDefinition->ApiVersion == ExpectedDefinitionApiVersion;
				UE_CLOGF(!bDefsApiVersionOk, LogOnlineServices, Warning, "EOS_Achievements_DefinitionV2 version mismatch Expected=%d Actual=%d", ExpectedDefinitionApiVersion, EOSDefinition->ApiVersion);

				FString AchievementId = UTF8_TO_TCHAR(EOSDefinition->AchievementId);
				FAchievementDefinition& AchievementDefinition = NewAchievementDefinitions.Emplace(AchievementId);
				AchievementDefinition.AchievementId = MoveTemp(AchievementId);
				AchievementDefinition.UnlockedDisplayName = FText::FromString(UTF8_TO_TCHAR(EOSDefinition->UnlockedDisplayName));
				AchievementDefinition.UnlockedDescription = FText::FromString(UTF8_TO_TCHAR(EOSDefinition->UnlockedDescription));
				AchievementDefinition.LockedDisplayName = FText::FromString(UTF8_TO_TCHAR(EOSDefinition->LockedDisplayName));
				AchievementDefinition.LockedDescription = FText::FromString(UTF8_TO_TCHAR(EOSDefinition->LockedDescription));
				AchievementDefinition.FlavorText = FText::FromString(UTF8_TO_TCHAR(EOSDefinition->FlavorText));
				AchievementDefinition.UnlockedIconUrl = UTF8_TO_TCHAR(EOSDefinition->UnlockedIconURL);
				AchievementDefinition.LockedIconUrl = UTF8_TO_TCHAR(EOSDefinition->LockedIconURL);
				AchievementDefinition.bIsHidden = EOSDefinition->bIsHidden == EOS_TRUE;
				for (uint32_t StatThresholdsIdx = 0; StatThresholdsIdx < EOSDefinition->StatThresholdsCount; StatThresholdsIdx++)
				{
					const EOS_Achievements_StatThresholds& EOSStatThreshold = EOSDefinition->StatThresholds[StatThresholdsIdx];

					const int ExpectedStatThresholdsApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_STATTHRESHOLDS_API_LATEST, ExpectedStatThresholdsApiVersion);
					const bool bStatsApiVersionOk = EOSStatThreshold.ApiVersion == ExpectedStatThresholdsApiVersion;
					UE_CLOGF(!bStatsApiVersionOk, LogOnlineServices, Warning, "EOS_Achievements_StatThresholds version mismatch Expected=%d Actual=%d", ExpectedStatThresholdsApiVersion, EOSStatThreshold.ApiVersion);

					FAchievementStatDefinition& StatDefinition = AchievementDefinition.StatDefinitions.Emplace_GetRef();
					StatDefinition.StatId = UTF8_TO_TCHAR(EOSStatThreshold.Name);
					StatDefinition.UnlockThreshold = uint32_t(EOSStatThreshold.Threshold);
				}
				EOS_Achievements_DefinitionV2_Release(EOSDefinition);
			}

			AchievementDefinitions.Emplace(MoveTemp(NewAchievementDefinitions));
			InAsyncOp.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FGetAchievementIds> FAchievementsEOSGS::GetAchievementIds(FGetAchievementIds::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetAchievementIds>(Errors::InvalidUser());
	}

	if (!AchievementDefinitions.IsSet())
	{
		// Call QueryAchievementDefinitions first
		return TOnlineResult<FGetAchievementIds>(Errors::InvalidState());
	}

	FGetAchievementIds::Result Result;
	AchievementDefinitions->GenerateKeyArray(Result.AchievementIds);
	return TOnlineResult<FGetAchievementIds>(MoveTemp(Result));
}

TOnlineResult<FGetAchievementDefinition> FAchievementsEOSGS::GetAchievementDefinition(FGetAchievementDefinition::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidUser());
	}

	if (!AchievementDefinitions.IsSet())
	{
		// Should call QueryAchievementDefinitions first
		return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidState());
	}

	const FAchievementDefinition* AchievementDefinition = AchievementDefinitions->Find(Params.AchievementId);
	if (!AchievementDefinition)
	{
		return TOnlineResult<FGetAchievementDefinition>(Errors::NotFound());
	}

	return TOnlineResult<FGetAchievementDefinition>({ *AchievementDefinition });
}

TOnlineAsyncOpHandle<FQueryAchievementStates> FAchievementsEOSGS::QueryAchievementStates(FQueryAchievementStates::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementStates> Op = GetJoinableOp<FQueryAchievementStates>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FQueryAchievementStates>& InAsyncOp)
		{
			const FQueryAchievementStates::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
			else if (!AchievementDefinitions.IsSet())
			{
				// Call QueryAchievementDefinitions first
				InAsyncOp.SetError(Errors::InvalidState());
			}
			else if (AchievementStates.Contains(Params.LocalAccountId))
			{
				InAsyncOp.SetResult({});
			}
		})
		.Then([this](TOnlineAsyncOp<FQueryAchievementStates>& InAsyncOp, TPromise<const EOS_Achievements_OnQueryPlayerAchievementsCompleteCallbackInfo*>&& Promise)
		{
			const FQueryAchievementStates::Params& Params = InAsyncOp.GetParams();

			EOS_Achievements_QueryPlayerAchievementsOptions Options = {};
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_QUERYPLAYERACHIEVEMENTS_API_LATEST, 2);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.TargetUserId = Options.LocalUserId;

			EOS_Async(EOS_Achievements_QueryPlayerAchievements, AchievementsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FQueryAchievementStates>& InAsyncOp, const EOS_Achievements_OnQueryPlayerAchievementsCompleteCallbackInfo* Data)
		{
			const FQueryAchievementStates::Params& Params = InAsyncOp.GetParams();

			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Achievements_QueryPlayerAchievements failed with result=[%ls]", *LexToString(Data->ResultCode));

				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			EOS_Achievements_GetPlayerAchievementCountOptions GetCountOptions = {};
			GetCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_GETPLAYERACHIEVEMENTCOUNT_API_LATEST, 1);
			GetCountOptions.UserId = GetProductUserIdChecked(Params.LocalAccountId);

			const uint32_t AchievementCount = EOS_Achievements_GetPlayerAchievementCount(AchievementsHandle, &GetCountOptions);

			FAchievementStateMap NewAchievementStates;
			for (uint32_t AchievementIdx = 0; AchievementIdx < AchievementCount; AchievementIdx++)
			{
				EOS_Achievements_CopyPlayerAchievementByIndexOptions CopyOptions = {};
				CopyOptions.ApiVersion = 2;
				UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_COPYPLAYERACHIEVEMENTBYINDEX_API_LATEST, 2);
				CopyOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
				CopyOptions.TargetUserId = CopyOptions.LocalUserId;
				CopyOptions.AchievementIndex = AchievementIdx;

				EOS_Achievements_PlayerAchievement* EOSPlayerAchievement = nullptr;
				const EOS_EResult CopyResult = EOS_Achievements_CopyPlayerAchievementByIndex(AchievementsHandle, &CopyOptions, &EOSPlayerAchievement);
				if (CopyResult != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_Achievements_CopyPlayerAchievementByIndex failed with result=[%ls]", *LexToString(CopyResult));

					InAsyncOp.SetError(Errors::FromEOSResult(CopyResult));
					return;
				}

				const int ExpectedPlayerAchievementApiVersion = 2;
				UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_PLAYERACHIEVEMENT_API_LATEST, ExpectedPlayerAchievementApiVersion);
				const bool bPlayerAchievementApiVersionOk = EOSPlayerAchievement->ApiVersion == ExpectedPlayerAchievementApiVersion;
				UE_CLOGF(!bPlayerAchievementApiVersionOk, LogOnlineServices, Warning, "EOS_Achievements_PlayerAchievement version mismatch Expected=%d Actual=%d", ExpectedPlayerAchievementApiVersion, EOSPlayerAchievement->ApiVersion);

				FString AchievementId = UTF8_TO_TCHAR(EOSPlayerAchievement->AchievementId);
				FAchievementState& AchievementState = NewAchievementStates.Emplace(AchievementId);
				AchievementState.AchievementId = MoveTemp(AchievementId);
				AchievementState.Progress = EOSPlayerAchievement->Progress;
				AchievementState.UnlockTime = FDateTime::FromUnixTimestamp(EOSPlayerAchievement->UnlockTime);

				EOS_Achievements_PlayerAchievement_Release(EOSPlayerAchievement);
			}
			AchievementStates.Emplace(Params.LocalAccountId, MoveTemp(NewAchievementStates));
			InAsyncOp.SetResult({});
			OnAchievementStatesQueried(Params.LocalAccountId);
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FGetAchievementState> FAchievementsEOSGS::GetAchievementState(FGetAchievementState::Params&& Params) const
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetAchievementState>(Errors::InvalidUser());
	}

	const FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
	if (!LocalUserAchievementStates)
	{
		// Call QueryAchievementStates first
		return TOnlineResult<FGetAchievementState>(Errors::InvalidState());
	}

	const FAchievementState* AchievementState = LocalUserAchievementStates->Find(Params.AchievementId);
	if (!AchievementState)
	{
		return TOnlineResult<FGetAchievementState>(Errors::NotFound());
	}

	return TOnlineResult<FGetAchievementState>({ *AchievementState });
}

TOnlineAsyncOpHandle<FUnlockAchievements> FAchievementsEOSGS::UnlockAchievements(FUnlockAchievements::Params&& Params)
{
	TOnlineAsyncOpRef<FUnlockAchievements> Op = GetJoinableOp<FUnlockAchievements>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FUnlockAchievements>& InAsyncOp)
		{
			const FUnlockAchievements::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
				return;
			}

			if (Params.AchievementIds.IsEmpty())
			{
				InAsyncOp.SetError(Errors::InvalidParams());
				return;
			}

			FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
			if (!LocalUserAchievementStates)
			{
				// Call QueryAchievementStates first
				InAsyncOp.SetError(Errors::InvalidState());
				return;
			}

			for (const FString& AchievementId : Params.AchievementIds)
			{
				const FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementId);
				if (!AchievementState)
				{
					InAsyncOp.SetError(Errors::NotFound());
					return;
				}
				if (AchievementState->Progress == 1.0f)
				{
					InAsyncOp.SetError(Errors::Achievements::AlreadyUnlocked());
					return;
				}
			}
		})
		.Then([this](TOnlineAsyncOp<FUnlockAchievements>& InAsyncOp, TPromise<const EOS_Achievements_OnUnlockAchievementsCompleteCallbackInfo*>&& Promise)
		{
			const FUnlockAchievements::Params& Params = InAsyncOp.GetParams();

			const TArray<FString>& InAchievementIds = Params.AchievementIds;
			TArray<FTCHARToUTF8> AchievementIdConverters;
			AchievementIdConverters.Reserve(InAchievementIds.Num());
			TArray<const char*> AchievementIdPtrs;
			AchievementIdPtrs.Reserve(InAchievementIds.Num());
			for (const FString& AchievementId : InAchievementIds)
			{
				const FTCHARToUTF8& Converter = AchievementIdConverters.Emplace_GetRef(*AchievementId);
				AchievementIdPtrs.Emplace(Converter.Get());
			}

			EOS_Achievements_UnlockAchievementsOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_API_LATEST, 1);
			Options.UserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.AchievementIds = AchievementIdPtrs.GetData();
			Options.AchievementsCount = AchievementIdPtrs.Num();

			EOS_Async(EOS_Achievements_UnlockAchievements, AchievementsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FUnlockAchievements>& InAsyncOp, const EOS_Achievements_OnUnlockAchievementsCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Achievements_UnlockAchievements failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			UE_LOGF(LogOnlineServices, Verbose, "EOS_Achievements_UnlockAchievements succeeded");

			const FUnlockAchievements::Params& Params = InAsyncOp.GetParams();
			const FDateTime TimeNow = FDateTime::Now();

			FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
			if (ensure(LocalUserAchievementStates))
			{
				for (const FString& AchievementId : Params.AchievementIds)
				{
					FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementId);
					if (ensure(AchievementState))
					{
						AchievementState->Progress = 1.0f;
						AchievementState->UnlockTime = TimeNow;
					}
				}
			}
			InAsyncOp.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

void FAchievementsEOSGS::HandleAchievementsUnlocked(const EOS_Achievements_OnAchievementsUnlockedCallbackV2Info* Data)
{
	const FAccountId LocalAccountId = FindAccountId(Data->UserId);
	if (LocalAccountId.IsValid())
	{
		const FString AchievementId = UTF8_TO_TCHAR(Data->AchievementId);
		const FDateTime UnlockTime = FDateTime::FromUnixTimestamp(Data->UnlockTime);

		UE_LOGF(LogOnlineServices, Verbose, "%s User=%ls AchievId=%ls", __FUNCTION__, *ToLogString(LocalAccountId), *AchievementId);

		if (FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(LocalAccountId))
		{
			if (FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementId))
			{
				AchievementState->Progress = 1.0f;
				AchievementState->UnlockTime = UnlockTime;
			}
		}
			
		FAchievementStateUpdated EventParams;
		EventParams.LocalAccountId = LocalAccountId;
		EventParams.AchievementIds.Emplace(AchievementId);

		OnAchievementStateUpdatedEvent.Broadcast(EventParams);
	}
}

FAccountId FAchievementsEOSGS::FindAccountId(const EOS_ProductUserId ProductUserId)
{
	return UE::Online::FindAccountId(Services.GetServicesProvider(), ProductUserId);
}

/* UE::Online */ }
