// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AuthEOSGS.h"

#include "Containers/BackgroundableTicker.h"
#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Misc/CommandLine.h"
#include "Online/AuthErrors.h"
#include "Online/EOSAuthLoginOptionsCommon.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"

#if __has_include(COMPILED_PLATFORM_HEADER(EOSAuthLoginOptions.h))
#include COMPILED_PLATFORM_HEADER(EOSAuthLoginOptions.h)
#else 
namespace UE::Online
{
    using FPlatformEOSAuthLoginOptions = FEOSAuthLoginOptionsCommon;
}
#endif

TAutoConsoleVariable<bool> CVarEnableExternalAuthUserAuthToken (
	TEXT("Online.EOS.EnableExternalAuthUserAuthToken"),
	false,
	TEXT("Configures GetExternalAuthTokenImpl to use EOS_Auth_CopyUserAuthToken when true, and EOS_Auth_CopyIdToken when false."),
	ECVF_Default);

#include "eos_auth.h"
#include "eos_connect.h"

namespace UE::Online {

struct FAuthEOSGSLoginConfig
{
	bool EASAuthEnabled = false;
	TArray<FString> DefaultScopes;
};

struct FAuthEOSGSLoginRecoveryConfig
{
	float Interval = 30.f;
	float Jitter = 5.f;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAuthEOSGSLoginConfig)
	ONLINE_STRUCT_FIELD(FAuthEOSGSLoginConfig, EASAuthEnabled),
	ONLINE_STRUCT_FIELD(FAuthEOSGSLoginConfig, DefaultScopes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthEOSGSLoginRecoveryConfig)
	ONLINE_STRUCT_FIELD(FAuthEOSGSLoginRecoveryConfig, Interval),
	ONLINE_STRUCT_FIELD(FAuthEOSGSLoginRecoveryConfig, Jitter)
END_ONLINE_STRUCT_META()

/* Meta*/ }

namespace {

static const FString AccountInfoKeyName = TEXT("AccountInfoEOS");

/* anonymous */ }

enum class EEOSConnectTranslationFlags : uint8
{
	None = 0,
	DisplayName = 1 << 0,
};
ENUM_CLASS_FLAGS(EEOSConnectTranslationFlags);

struct FEOSConnectTranslationTraits
{
	//FEOSConnectTranslationTraits() = delete;
	EOS_EExternalCredentialType Type;
	EEOSConnectTranslationFlags Flags;
};

class FEOSConnectLoginOptions : public EOS_Connect_LoginOptions
{
public:
	FEOSConnectLoginOptions(const FEOSConnectLoginOptions&) = delete;
	FEOSConnectLoginOptions& operator=(const FEOSConnectLoginOptions&) = delete;
	FEOSConnectLoginOptions(FEOSConnectLoginOptions&&);
	FEOSConnectLoginOptions& operator=(FEOSConnectLoginOptions&&);
	virtual ~FEOSConnectLoginOptions() = default;

	static TDefaultErrorResultInternal<FEOSConnectLoginOptions> Create(FPlatformUserId PlatformUserId, const FExternalAuthToken& ExternalAuthToken);
private:
	FEOSConnectLoginOptions();

	static const FEOSConnectTranslationTraits* GetConnectTranslationTraits(FName ExternalLoginType);

	EOS_Connect_Credentials CredentialsData;
	EOS_Connect_UserLoginInfo UserLoginInfoData;
	char DisplayNameUtf8[EOS_CONNECT_USERLOGININFO_DISPLAYNAME_MAX_LENGTH + 1] = {};
	TArray<char> ExternalAuthTokenUtf8;
};

const FEOSConnectTranslationTraits* FEOSConnectLoginOptions::GetConnectTranslationTraits(FName ExternalLoginType)
{
	static const TMap<FName, FEOSConnectTranslationTraits> SupportedLoginTranslatorTraits = {
		{ ExternalLoginType::Epic, { EOS_EExternalCredentialType::EOS_ECT_EPIC, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::SteamSessionTicket, { EOS_EExternalCredentialType::EOS_ECT_STEAM_SESSION_TICKET, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::PsnIdToken, { EOS_EExternalCredentialType::EOS_ECT_PSN_ID_TOKEN, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::XblXstsToken, { EOS_EExternalCredentialType::EOS_ECT_XBL_XSTS_TOKEN, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::DiscordAccessToken, { EOS_EExternalCredentialType::EOS_ECT_DISCORD_ACCESS_TOKEN, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::GogSessionTicket, { EOS_EExternalCredentialType::EOS_ECT_GOG_SESSION_TICKET, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::NintendoIdToken, { EOS_EExternalCredentialType::EOS_ECT_NINTENDO_ID_TOKEN, EEOSConnectTranslationFlags::DisplayName } },
		{ ExternalLoginType::NintendoNsaIdToken, { EOS_EExternalCredentialType::EOS_ECT_NINTENDO_NSA_ID_TOKEN, EEOSConnectTranslationFlags::DisplayName } },
		{ ExternalLoginType::UplayAccessToken, { EOS_EExternalCredentialType::EOS_ECT_UPLAY_ACCESS_TOKEN, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::OpenIdAccessToken, { EOS_EExternalCredentialType::EOS_ECT_OPENID_ACCESS_TOKEN, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::DeviceIdAccessToken, { EOS_EExternalCredentialType::EOS_ECT_DEVICEID_ACCESS_TOKEN, EEOSConnectTranslationFlags::DisplayName } },
		{ ExternalLoginType::AppleIdToken, { EOS_EExternalCredentialType::EOS_ECT_APPLE_ID_TOKEN, EEOSConnectTranslationFlags::DisplayName } },
		{ ExternalLoginType::GoogleIdToken, { EOS_EExternalCredentialType::EOS_ECT_GOOGLE_ID_TOKEN, EEOSConnectTranslationFlags::DisplayName } },
		{ ExternalLoginType::OculusUserIdNonce, { EOS_EExternalCredentialType::EOS_ECT_OCULUS_USERID_NONCE, EEOSConnectTranslationFlags::DisplayName } },
		{ ExternalLoginType::ItchioJwt, { EOS_EExternalCredentialType::EOS_ECT_ITCHIO_JWT, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::ItchioKey, { EOS_EExternalCredentialType::EOS_ECT_ITCHIO_KEY, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::EpicIdToken, { EOS_EExternalCredentialType::EOS_ECT_EPIC_ID_TOKEN, EEOSConnectTranslationFlags::None } },
		{ ExternalLoginType::AmazonAccessToken, { EOS_EExternalCredentialType::EOS_ECT_AMAZON_ACCESS_TOKEN, EEOSConnectTranslationFlags::DisplayName } },
	};

	return SupportedLoginTranslatorTraits.Find(ExternalLoginType);
}

FEOSConnectLoginOptions::FEOSConnectLoginOptions(FEOSConnectLoginOptions&& Other)
{
	*this = MoveTemp(Other);
}

FEOSConnectLoginOptions& FEOSConnectLoginOptions::operator=(FEOSConnectLoginOptions&& Other)
{
	ApiVersion = Other.ApiVersion;
	Credentials = nullptr;
	UserLoginInfo = nullptr;

	if (Other.Credentials)
	{
		ExternalAuthTokenUtf8 = MoveTemp(Other.ExternalAuthTokenUtf8);
		CredentialsData = Other.CredentialsData;
		CredentialsData.Token = ExternalAuthTokenUtf8.GetData();
		Credentials = &CredentialsData;
		Other.CredentialsData = {};
	}

	if (Other.UserLoginInfo)
	{
		memcpy(DisplayNameUtf8, Other.DisplayNameUtf8, sizeof(DisplayNameUtf8));
		UserLoginInfoData = Other.UserLoginInfoData;
		UserLoginInfoData.DisplayName = DisplayNameUtf8;
		UserLoginInfo = &UserLoginInfoData;
		Other.UserLoginInfo = nullptr;
		Other.UserLoginInfoData = {};
	}

	return *this;
}

FEOSConnectLoginOptions::FEOSConnectLoginOptions()
{
	// EOS_Connect_LoginOptions init
	ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_LOGIN_API_LATEST, 2);
	Credentials = &CredentialsData;
	UserLoginInfo = nullptr;

	// EOS_Connect_Credentials init
	CredentialsData.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_CREDENTIALS_API_LATEST, 1);
	CredentialsData.Token = nullptr;

	// EOS_Connect_UserLoginInfo init
	UserLoginInfoData.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_USERLOGININFO_API_LATEST, 2);
	UserLoginInfoData.DisplayName = DisplayNameUtf8;
	UserLoginInfoData.NsaIdToken = nullptr;
}

TDefaultErrorResultInternal<FEOSConnectLoginOptions> FEOSConnectLoginOptions::Create(FPlatformUserId PlatformUserId, const FExternalAuthToken& ExternalAuthToken)
{
	const FEOSConnectTranslationTraits* TranslatorTraits = GetConnectTranslationTraits(ExternalAuthToken.Type);
	if (TranslatorTraits == nullptr)
	{
		UE_LOGF(LogOnlineServices, Warning, "FEOSConnectLoginOptions::Create: Failed - Unsupported external auth type: %ls", *ExternalAuthToken.Type.ToString());
		return TDefaultErrorResultInternal<FEOSConnectLoginOptions>(Errors::InvalidParams());
	}

	FEOSConnectLoginOptions EOSConnectLoginOptions;
	if (EnumHasAnyFlags(TranslatorTraits->Flags, EEOSConnectTranslationFlags::DisplayName))
	{
		// Todo: Lookup platform OSS Auth interface to resolve display name using PlatformUserId.
		FCStringAnsi::Strncpy(EOSConnectLoginOptions.DisplayNameUtf8, "PlatformUser", sizeof(EOSConnectLoginOptions.DisplayNameUtf8));
		EOSConnectLoginOptions.UserLoginInfo = &EOSConnectLoginOptions.UserLoginInfoData;
	}

	EOSConnectLoginOptions.ExternalAuthTokenUtf8.SetNumZeroed(ExternalAuthToken.Data.Len() + 1);
	FCStringAnsi::Strncpy(EOSConnectLoginOptions.ExternalAuthTokenUtf8.GetData(), TCHAR_TO_UTF8(*ExternalAuthToken.Data), EOSConnectLoginOptions.ExternalAuthTokenUtf8.Num());
	EOSConnectLoginOptions.CredentialsData.Token = EOSConnectLoginOptions.ExternalAuthTokenUtf8.GetData();
	EOSConnectLoginOptions.CredentialsData.Type = TranslatorTraits->Type;
	return TDefaultErrorResultInternal<FEOSConnectLoginOptions>(MoveTemp(EOSConnectLoginOptions));
}

class FEOSAuthLoginOptions : public FPlatformEOSAuthLoginOptions
{
public:
	static TDefaultErrorResultInternal<FEOSAuthLoginOptions> Create(
		FName CredentialsType,
		const FString& CredentialsId,
		const TVariant<FString, FExternalAuthToken>& CredentialsToken,
		const TArray<FString>& Scopes);

private:
	static TDefaultErrorResultInternal<FEOSAuthLoginOptions> CreateImpl(
		FName CredentialsType,
		const FString& CredentialsId,
		const TVariant<FString, FExternalAuthToken>& CredentialsToken,
		const TArray<FString>& Scopes);

	FEOSAuthLoginOptions() = default;
};

TDefaultErrorResultInternal<FEOSAuthLoginOptions> FEOSAuthLoginOptions::Create(
	FName CredentialsType,
	const FString& CredentialsId,
	const TVariant<FString, FExternalAuthToken>& CredentialsToken,
	const TArray<FString>& Scopes)
{
	if (CredentialsType == LoginCredentialsType::Auto)
	{
		FString	CommandLineAuthType;
		FString	CommandLineAuthId;
		FString	CommandLineAuthToken;
		FParse::Value(FCommandLine::Get(), TEXT("AUTH_TYPE="), CommandLineAuthType);
		FParse::Value(FCommandLine::Get(), TEXT("AUTH_LOGIN="), CommandLineAuthId);
		FParse::Value(FCommandLine::Get(), TEXT("AUTH_PASSWORD="), CommandLineAuthToken);
		if (!CommandLineAuthId.IsEmpty() && !CommandLineAuthToken.IsEmpty() && !CommandLineAuthType.IsEmpty())
		{
			return CreateImpl(
				*CommandLineAuthType,
				CommandLineAuthId,
				TVariant<FString, FExternalAuthToken>(TInPlaceType<FString>(),
				MoveTemp(CommandLineAuthToken)),
				Scopes);
		}
		else
		{
			return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidCreds());
		}
	}
	else
	{
		return CreateImpl(CredentialsType, CredentialsId, CredentialsToken, Scopes);
	}
}

TDefaultErrorResultInternal<FEOSAuthLoginOptions> FEOSAuthLoginOptions::CreateImpl(
	FName CredentialsType,
	const FString& CredentialsId,
	const TVariant<FString, FExternalAuthToken>& CredentialsToken,
	const TArray<FString>& Scopes)
{
	const FEOSAuthTranslationTraits* TranslatorTraits = GetLoginTranslatorTraits(CredentialsType);
	if (TranslatorTraits == nullptr)
	{
		UE_LOGF(LogOnlineServices, Warning, "FEOSAuthLoginOptions::Create: Failed - Unsupported login type: %ls", *CredentialsType.ToString());
		return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidParams());
	}

	FEOSAuthLoginOptions EOSAuthLoginOptions;
	EOSAuthLoginOptions.CredentialsData.Type = TranslatorTraits->Type;

	if (EnumHasAnyFlags(TranslatorTraits->Flags, EEOSAuthTranslationFlags::SetId))
	{
		const FTCHARToUTF8 Converter(*CredentialsId);
		EOSAuthLoginOptions.IdUtf8.SetNumZeroed(Converter.Length() + 1);
		FCStringAnsi::Strncpy(EOSAuthLoginOptions.IdUtf8.GetData(), Converter.Get(), EOSAuthLoginOptions.IdUtf8.Num());
		EOSAuthLoginOptions.CredentialsData.Id = EOSAuthLoginOptions.IdUtf8.GetData();
	}

	if (EnumHasAnyFlags(TranslatorTraits->Flags, EEOSAuthTranslationFlags::SetTokenFromString))
	{
		if (!CredentialsToken.IsType<FString>())
		{
			UE_LOGF(LogOnlineServices, Warning, "FEOSAuthLoginOptions::Create: Failed - Expected CredentialsToken set to string for Login type: %ls", *CredentialsType.ToString());
			return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidCreds());
		}

		const FTCHARToUTF8 Converter(*CredentialsToken.Get<FString>());
		EOSAuthLoginOptions.TokenUtf8.SetNumZeroed(Converter.Length() + 1);
		FCStringAnsi::Strncpy(EOSAuthLoginOptions.TokenUtf8.GetData(), Converter.Get(), EOSAuthLoginOptions.TokenUtf8.Num());
		EOSAuthLoginOptions.CredentialsData.Token = EOSAuthLoginOptions.TokenUtf8.GetData();
	}

	if (EnumHasAnyFlags(TranslatorTraits->Flags, EEOSAuthTranslationFlags::SetTokenFromExternalAuth))
	{
		if (!CredentialsToken.IsType<FExternalAuthToken>())
		{
			UE_LOGF(LogOnlineServices, Warning, "FEOSAuthLoginOptions::Create: Failed - Expected CredentialsToken set to ExternalAuthToken for Login type: %ls", *CredentialsType.ToString());
			return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidCreds());
		}

		const FExternalAuthToken& ExternalAuthToken = CredentialsToken.Get<FExternalAuthToken>();
		const FEOSExternalAuthTranslationTraits* ExternalAuthTranslationTraits = GetExternalAuthTranslationTraits(ExternalAuthToken.Type);
		if (ExternalAuthTranslationTraits == nullptr)
		{
			UE_LOGF(LogOnlineServices, Warning, "FEOSAuthLoginOptions::Create: Failed - Unsuppoerted external auth type: %ls", *ExternalAuthToken.Type.ToString());
			return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidCreds());
		}

		EOSAuthLoginOptions.TokenUtf8.SetNumZeroed(ExternalAuthToken.Data.Len() + 1);
		FCStringAnsi::Strncpy(EOSAuthLoginOptions.TokenUtf8.GetData(), TCHAR_TO_UTF8(*ExternalAuthToken.Data), EOSAuthLoginOptions.TokenUtf8.Num());
		EOSAuthLoginOptions.CredentialsData.Token = EOSAuthLoginOptions.TokenUtf8.GetData();
		EOSAuthLoginOptions.CredentialsData.ExternalType = ExternalAuthTranslationTraits->Type;
		EOSAuthLoginOptions.LinkAccountFlags = ExternalAuthTranslationTraits->LinkAccountFlags;

		UE_LOGF(LogOnlineServices, VeryVerbose, "FEOSAuthLoginOptions::Create: Using token type: %ls, Token data: %.*ls", *ExternalAuthToken.Type.ToString(), ExternalAuthToken.Data.Len(), *ExternalAuthToken.Data);
	}

	if (!InitSystemAuthCredentialOptions(EOSAuthLoginOptions))
	{
		UE_LOGF(LogOnlineServices, Warning, "FEOSAuthLoginOptions::Create: Failed - Failed to initialize system credential options");
		return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidCreds());
	}

	// Translate scopes.
	bool bAllScopesValid = true;
	for (const FString& Scope : Scopes)
	{
		EOS_EAuthScopeFlags ScopeFlags;
		if (LexFromString(ScopeFlags, Scope))
		{
			EOSAuthLoginOptions.ScopeFlags |= ScopeFlags;
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "FEOSAuthLoginOptions::Create: Failed - Unknown scope: %ls", *Scope);
			bAllScopesValid = false;
		}
	}
	if (!bAllScopesValid)
	{
		return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(Errors::InvalidCreds());
	}

	return TDefaultErrorResultInternal<FEOSAuthLoginOptions>(MoveTemp(EOSAuthLoginOptions));
}

TSharedPtr<FAccountInfoEOS> FAccountInfoRegistryEOS::Find(FPlatformUserId PlatformUserId) const
{
	return StaticCastSharedPtr<FAccountInfoEOS>(Super::Find(PlatformUserId));
}

TSharedPtr<FAccountInfoEOS> FAccountInfoRegistryEOS::Find(FAccountId AccountId) const
{
	return StaticCastSharedPtr<FAccountInfoEOS>(Super::Find(AccountId));
}

TSharedPtr<FAccountInfoEOS> FAccountInfoRegistryEOS::Find(EOS_EpicAccountId EpicAccountId) const
{
	FReadScopeLock Lock(IndexLock);
	if (const TSharedRef<FAccountInfoEOS>* FoundPtr = AuthDataByEpicAccountId.Find(EpicAccountId))
	{
		return *FoundPtr;
	}
	else
	{
		return nullptr;
	}
}

TSharedPtr<FAccountInfoEOS> FAccountInfoRegistryEOS::Find(EOS_ProductUserId ProductUserId) const
{
	FReadScopeLock Lock(IndexLock);
	if (const TSharedRef<FAccountInfoEOS>* FoundPtr = AuthDataByProductUserId.Find(ProductUserId))
	{
		return *FoundPtr;
	}
	else
	{
		return nullptr;
	}
}

void FAccountInfoRegistryEOS::Register(const TSharedRef<FAccountInfoEOS>& AccountInfoEOS)
{
	FWriteScopeLock Lock(IndexLock);
	DoRegister(AccountInfoEOS);
}

void FAccountInfoRegistryEOS::Unregister(FAccountId AccountId)
{
	if (TSharedPtr<FAccountInfoEOS> AccountInfoEOS = Find(AccountId))
	{
		FWriteScopeLock Lock(IndexLock);
		DoUnregister(AccountInfoEOS.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FAccountInfoRegistryEOS::Unregister] Failed to find account [%ls].", *ToLogString(AccountId));
	}
}

void FAccountInfoRegistryEOS::DoRegister(const TSharedRef<FAccountInfo>& AccountInfo)
{
	TSharedRef<FAccountInfoEOS> AccountInfoEOS = StaticCastSharedRef<FAccountInfoEOS>(AccountInfo);

	Super::DoRegister(AccountInfo);

	if (AccountInfoEOS->EpicAccountId)
	{
		AuthDataByEpicAccountId.Add(AccountInfoEOS->EpicAccountId, AccountInfoEOS);
	}

	if (AccountInfoEOS->ProductUserId)
	{
		AuthDataByProductUserId.Add(AccountInfoEOS->ProductUserId, AccountInfoEOS);
	}
}
void FAccountInfoRegistryEOS::DoUnregister(const TSharedRef<FAccountInfo>& AccountInfo)
{
	TSharedRef<FAccountInfoEOS> AccountInfoEOS = StaticCastSharedRef<FAccountInfoEOS>(AccountInfo);

	Super::DoUnregister(AccountInfo);

	if (AccountInfoEOS->EpicAccountId)
	{
		AuthDataByEpicAccountId.Remove(AccountInfoEOS->EpicAccountId);
	}

	if (AccountInfoEOS->ProductUserId)
	{
		AuthDataByProductUserId.Remove(AccountInfoEOS->ProductUserId);
	}
}

FAuthEOSGS::FAuthEOSGS(FOnlineServicesCommon& InServices)
	: Super(InServices)
{
}

void FAuthEOSGS::Initialize()
{
	Super::Initialize();

	AuthHandle = EOS_Platform_GetAuthInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(AuthHandle != nullptr);

	ConnectHandle = EOS_Platform_GetConnectInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(ConnectHandle != nullptr);

	RegisterHandlers();

}

void FAuthEOSGS::PreShutdown()
{
	Super::PreShutdown();

	UnregisterHandlers();
}

TOnlineAsyncOpHandle<FAuthLogin> FAuthEOSGS::Login(FAuthLogin::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLogin> Op = GetOp<FAuthLogin>(MoveTemp(Params));

	// Login in AuthEOSGS supports calling "Login" again with an updated external auth token to
	// avoid service interruption when the connect token expires.

	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const FAuthLogin::Params& Params = InAsyncOp.GetParams();

		// Check that user is valid.
		if (!Params.PlatformUserId.IsValid())
		{
			InAsyncOp.SetError(Errors::InvalidParams());
			return;
		}

		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.PlatformUserId);
		if (!AccountInfoEOS)
		{
			AccountInfoEOS = MakeShared<FAccountInfoEOS>();
			AccountInfoEOS->PlatformUserId = Params.PlatformUserId;
			AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(AccountInfoKeyName, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Optionally handle EAS login.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);
		FAuthEOSGSLoginConfig AuthEOSGSLoginConfig;
		LoadConfig(AuthEOSGSLoginConfig, TEXT("Login"));
		
		if (AuthEOSGSLoginConfig.EASAuthEnabled)
		{
			if (!IsOnlineStatus(AccountInfoEOS->LoginStatus))
			{
				TPromise<void> Promise;

				const FAuthLogin::Params& Params = InAsyncOp.GetParams();

				FAuthLoginEASImpl::Params LoginParams;
				LoginParams.PlatformUserId = Params.PlatformUserId;
				LoginParams.CredentialsType = Params.CredentialsType;
				LoginParams.CredentialsId = Params.CredentialsId;
				LoginParams.CredentialsToken = Params.CredentialsToken;
				LoginParams.Scopes = !Params.Scopes.IsEmpty() ? Params.Scopes : AuthEOSGSLoginConfig.DefaultScopes;

				return LoginEASImpl(LoginParams)
				.Next([this, Promise = MoveTemp(Promise), Op = InAsyncOp.AsShared()](TDefaultErrorResult<FAuthLoginEASImpl>&& LoginResult) mutable -> void
				{
					const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(*Op, AccountInfoKeyName);

					if (LoginResult.IsError())
					{
						UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::Login] Failure: LoginEASImpl %ls", *LoginResult.GetErrorValue().GetLogString());
						Op->SetError(MoveTemp(LoginResult.GetErrorValue()));
					}
					else
					{
						// Cache EpicAccountId on successful EAS login.
						AccountInfoEOS->EpicAccountId = LoginResult.GetOkValue().EpicAccountId;
					}

					Promise.EmplaceValue();
				});
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::Login] Failure: EAS login enabled and already logged in.");
				InAsyncOp.SetError(Errors::Auth::AlreadyLoggedIn());
			}
		}

		return MakeFulfilledPromise<void>().GetFuture();
	})
	// Step 3: Fetch external auth credentials for connect login.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const FAuthLogin::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		TPromise<FAuthLoginConnectImpl::Params> Promise;
		TFuture<FAuthLoginConnectImpl::Params> Future = Promise.GetFuture();

		// When an epic account is present, use the external auth credentials of the logged in account.
		if (AccountInfoEOS->EpicAccountId)
		{
			TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> AuthTokenResult = GetExternalAuthTokenImpl(FAuthGetExternalAuthTokenImpl::Params{AccountInfoEOS->EpicAccountId});
			if (AuthTokenResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::Login] Failure: GetExternalAuthTokenImpl %ls", *AuthTokenResult.GetErrorValue().GetLogString());

				// Failed to acquire token - logout EAS.
				LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
				.Next([AsyncOp = InAsyncOp.AsShared(), Error = MoveTemp(AuthTokenResult.GetErrorValue()), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
				{
					AsyncOp->SetError(MoveTemp(Error));
					Promise.EmplaceValue(FAuthLoginConnectImpl::Params{});
				});
				return Future;
			}

			Promise.EmplaceValue(FAuthLoginConnectImpl::Params{Params.PlatformUserId, MoveTemp(AuthTokenResult.GetOkValue().Token)});
		}
		else
		{
			// Connect login requires external auth.
			if (Params.CredentialsType != LoginCredentialsType::ExternalAuth || !Params.CredentialsToken.IsType<FExternalAuthToken>())
			{
				InAsyncOp.SetError(Errors::InvalidParams());
				Promise.EmplaceValue(FAuthLoginConnectImpl::Params{});
				return Future;
			}

			Promise.EmplaceValue(FAuthLoginConnectImpl::Params{Params.PlatformUserId, Params.CredentialsToken.Get<FExternalAuthToken>()});
		}

		return Future;
	})
	// Step 4: Attempt connect login. On connect login failure handle logout of EAS.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp, FAuthLoginConnectImpl::Params&& LoginConnectParams)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		// Attempt connect login.
		LoginConnectImpl(LoginConnectParams)
		.Next([this, AccountInfoEOS, AsyncOp = InAsyncOp.AsShared(), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLoginConnectImpl>&& LoginResult) mutable -> void
		{
			if (LoginResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::Login] Failure: LoginConnectImpl %ls", *LoginResult.GetErrorValue().GetLogString());

				// Logout of EAS on login failure if necessary.
				if (AccountInfoEOS->EpicAccountId)
				{
					LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
					.Next([AsyncOp = AsyncOp->AsShared(), Error = MoveTemp(LoginResult.GetErrorValue()), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
					{
						AsyncOp->SetError(MoveTemp(Error));
						Promise.EmplaceValue();
					});
				}
				else
				{
					AsyncOp->SetError(MoveTemp(LoginResult.GetErrorValue()));
					Promise.EmplaceValue();
				}
			}
			else
			{
				// Successful login.
				AccountInfoEOS->ProductUserId = LoginResult.GetOkValue().ProductUserId;
				Promise.EmplaceValue();
			}
		});

		return Future;
	})
	// Step 5: Fetch dependent data.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		if (!IsOnlineStatus(AccountInfoEOS->LoginStatus))
		{
			// Set DisplayName.
			int32_t ProductUserIdBufferLength = EOS_PRODUCTUSERID_MAX_LENGTH + 1;
			char ProductUserIdBuffer[EOS_PRODUCTUSERID_MAX_LENGTH + 1] = {};
			EOS_EResult ProductUserIdResult = EOS_ProductUserId_ToString(AccountInfoEOS->ProductUserId, ProductUserIdBuffer, &ProductUserIdBufferLength);
			if (ProductUserIdResult == EOS_EResult::EOS_Success)
			{
				AccountInfoEOS->Attributes.Emplace(AccountAttributeData::DisplayName, UTF8_TO_TCHAR(ProductUserIdBuffer));
			}
			else
			{
				FOnlineError ProductUserIdError(Errors::FromEOSResult(ProductUserIdResult));
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::Login] Failure: EOS_ProductUserId_ToString %ls", *ProductUserIdError.GetLogString());

				// Handle EAS logout if needed.
				if (AccountInfoEOS->EpicAccountId)
				{
					TPromise<void> Promise;
					TFuture<void> Future = Promise.GetFuture();
					LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
					.Next([AsyncOp = InAsyncOp.AsShared(), Error = MoveTemp(ProductUserIdError), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
					{
						AsyncOp->SetError(MoveTemp(Error));
						Promise.EmplaceValue();
					});
					return Future;
				}
				else
				{
					InAsyncOp.SetError(MoveTemp(ProductUserIdError));
				}
			}
		}

		return MakeFulfilledPromise<void>().GetFuture();
	})
	// Step 6: bookkeeping and notifications.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		if (IsOnlineStatus(AccountInfoEOS->LoginStatus))
		{
			UE_LOGF(LogOnlineServices, Log, "[FAuthEOSGS::Login] Successfully refreshed logged in as [%ls]", *ToLogString(AccountInfoEOS->AccountId));
		}
		else
		{
			AccountInfoEOS->LoginStatus = ELoginStatus::LoggedIn;
			AccountInfoEOS->AccountId = CreateAccountId(AccountInfoEOS->ProductUserId);
			AccountInfoRegistryEOS.Register(AccountInfoEOS);

			UE_LOGF(LogOnlineServices, Log, "[FAuthEOSGS::Login] Successfully logged in as [%ls]", *ToLogString(AccountInfoEOS->AccountId));
			OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });
		}

		InAsyncOp.SetResult(FAuthLogin::Result{AccountInfoEOS});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthLogout> FAuthEOSGS::Logout(FAuthLogout::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLogout> Op = GetOp<FAuthLogout>(MoveTemp(Params));

	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const FAuthLogout::Params& Params = InAsyncOp.GetParams();
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalAccountId);
		if (!AccountInfoEOS)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(AccountInfoKeyName, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Delete persistent auth if needed.
	.Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const FAuthLogout::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		if (Params.bDestroyAuth && AccountInfoEOS->EpicAccountId)
		{
			TPromise<void> Promise;
			TFuture<void> Future = Promise.GetFuture();

			EOS_Auth_DeletePersistentAuthOptions DeletePersistentAuthOptions = { 0 };
			DeletePersistentAuthOptions.ApiVersion = 2;
			DeletePersistentAuthOptions.RefreshToken = nullptr; // TODO: Is this needed?  Docs say it's needed for consoles
			UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_DELETEPERSISTENTAUTH_API_LATEST, 2);

			EOS_Async(EOS_Auth_DeletePersistentAuth, AuthHandle, DeletePersistentAuthOptions,
			[Promise = MoveTemp(Promise)](const EOS_Auth_DeletePersistentAuthCallbackInfo* Data) mutable -> void
			{
				if (Data->ResultCode != EOS_EResult::EOS_Success)
				{
					FOnlineError DeletePersistentAuthError(Errors::FromEOSResult(Data->ResultCode));
					UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::Logout] Failure: DeletePersistentAuthResult %ls", *DeletePersistentAuthError.GetLogString());
				}

				Promise.EmplaceValue();
			});

			return Future;
		}

		return MakeFulfilledPromise<void>().GetFuture();
	})
	// Step 3: Logout of EAS if necessary.
	.Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		if (AccountInfoEOS->EpicAccountId)
		{
			TPromise<void> Promise;
			TFuture<void> Future = Promise.GetFuture();
			LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
			.Next([Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
			{
				Promise.EmplaceValue();
			});
			return Future;
		}
		else
		{
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	// Step 4: Logout of Connect.
	.Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		if (AccountInfoEOS->ProductUserId)
		{
			TPromise<void> Promise;
			TFuture<void> Future = Promise.GetFuture();
			LogoutConnectImpl(FAuthLogoutConnectImpl::Params{ AccountInfoEOS->ProductUserId })
			.Next([Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutConnectImpl>&&) mutable -> void
			{
				Promise.EmplaceValue();
			});
			return Future;
		}
		else
		{
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	.Then([this](TOnlineAsyncOp<FAuthLogout>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		UE_LOGF(LogOnlineServices, Log, "[FAuthEOSGS::Logout] Successfully logged out [%ls]", *ToLogString(AccountInfoEOS->AccountId));
		AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });
		AccountInfoRegistryEOS.Unregister(AccountInfoEOS->AccountId);

		InAsyncOp.SetResult(FAuthLogout::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryVerifiedAuthTicket> FAuthEOSGS::QueryVerifiedAuthTicket(FAuthQueryVerifiedAuthTicket::Params&& Params)
{
	// Todo
	TOnlineAsyncOpRef<FAuthQueryVerifiedAuthTicket> Operation = GetOp<FAuthQueryVerifiedAuthTicket>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthCancelVerifiedAuthTicket> FAuthEOSGS::CancelVerifiedAuthTicket(FAuthCancelVerifiedAuthTicket::Params&& Params)
{
	// Todo
	TOnlineAsyncOpRef<FAuthCancelVerifiedAuthTicket> Operation = GetOp<FAuthCancelVerifiedAuthTicket>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthBeginVerifiedAuthSession> FAuthEOSGS::BeginVerifiedAuthSession(FAuthBeginVerifiedAuthSession::Params&& Params)
{
	// Todo
	TOnlineAsyncOpRef<FAuthBeginVerifiedAuthSession> Operation = GetOp<FAuthBeginVerifiedAuthSession>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FAuthGetRelyingParty> FAuthEOSGS::GetRelyingParty(FAuthGetRelyingParty::Params&& Params) const
{
	IEOSSDKManager* Manager = IEOSSDKManager::Get();
	const FString& PlatformConfigName = GetServices<FOnlineServicesEOSGS>().GetEOSPlatformHandle()->GetConfigName();
	const FEOSSDKPlatformConfig* Config = Manager ? Manager->GetPlatformConfig(PlatformConfigName) : nullptr;
	if (Config == nullptr)
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: No configuration found for EOS platform %ls.", __FUNCTION__, *PlatformConfigName);
		return TOnlineResult<FAuthGetRelyingParty>(Errors::NotConfigured());
	}

	FString OutRelyingParty = Config->RelyingPartyURI;
	if (OutRelyingParty.IsEmpty())
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Relying party configuration is missing for sandbox %ls", __FUNCTION__, *Config->SandboxId);
		return TOnlineResult<FAuthGetRelyingParty>(Errors::NotConfigured());
	}

	UE_LOGF(LogOnlineServices, VeryVerbose, "[%s]: Using relying party: %ls", __FUNCTION__, *OutRelyingParty);
	return TOnlineResult<FAuthGetRelyingParty>(FAuthGetRelyingParty::Result{MoveTemp(OutRelyingParty)});
}

TOnlineAsyncOpHandle<FAuthEndVerifiedAuthSession> FAuthEOSGS::EndVerifiedAuthSession(FAuthEndVerifiedAuthSession::Params&& Params)
{
	// Todo
	TOnlineAsyncOpRef<FAuthEndVerifiedAuthSession> Operation = GetOp<FAuthEndVerifiedAuthSession>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TFuture<TDefaultErrorResult<FAuthLoginEASImpl>> FAuthEOSGS::LoginEASImpl(const FAuthLoginEASImpl::Params& LoginParams)
{
	TDefaultErrorResultInternal<FEOSAuthLoginOptions> LoginOptionsResult = FEOSAuthLoginOptions::Create(LoginParams.CredentialsType, LoginParams.CredentialsId, LoginParams.CredentialsToken, LoginParams.Scopes);
	if (LoginOptionsResult.IsError())
	{
		return MakeFulfilledPromise<TDefaultErrorResult<FAuthLoginEASImpl>>(MoveTemp(LoginOptionsResult.GetErrorValue())).GetFuture();
	}

	const bool bIsPersistentAuthLogin = LoginParams.CredentialsType == LoginCredentialsType::PersistentAuth;
	const EOS_ELinkAccountFlags LinkAccountFlags = LoginOptionsResult.GetOkValue().GetLinkAccountFlags();

	TPromise<TDefaultErrorResult<FAuthLoginEASImpl>> Promise;
	TFuture<TDefaultErrorResult<FAuthLoginEASImpl>> Future = Promise.GetFuture();
    
	FEOSAuthLoginOptions LoginOptions = MoveTemp(LoginOptionsResult.GetOkValue());
    
#if PLATFORM_IOS
	const EDeviceScreenOrientation OriginalDeviceOrientation = FPlatformMisc::GetAllowedDeviceOrientation();
	// TODO move this into ASWebAuthenticationSession callbacks when they become available.
	if (LoginOptions.Credentials->Type == EOS_ELoginCredentialType::EOS_LCT_AccountPortal)
	{
		FPlatformMisc::SetAllowedDeviceOrientation(EDeviceScreenOrientation::FullSensor);
	}
#endif

	EOS_Async(EOS_Auth_Login, AuthHandle, MoveTemp(LoginOptions),
	[this, bIsPersistentAuthLogin, PlatformUserId = LoginParams.PlatformUserId, LinkAccountFlags, bAutoLinkAccount = LoginParams.bAutoLinkAccount, Promise = MoveTemp(Promise)
#if PLATFORM_IOS
		, OriginalDeviceOrientation
#endif
	](const EOS_Auth_LoginCallbackInfo* Data) mutable -> void
	{
#if PLATFORM_IOS
        // TODO move this into ASWebAuthenticationSession callbacks when they become available.
		if (FPlatformMisc::GetAllowedDeviceOrientation() != OriginalDeviceOrientation)
		{
			FPlatformMisc::SetAllowedDeviceOrientation(OriginalDeviceOrientation);
		}
#endif
        
		UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LoginEASImpl] EOS_Auth_Login Result: [%ls]", *LexToString(Data->ResultCode));

		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			Promise.SetValue(TDefaultErrorResult<FAuthLoginEASImpl>(FAuthLoginEASImpl::Result{ Data->LocalUserId }));
		}
		else if (Data->ResultCode == EOS_EResult::EOS_InvalidUser && Data->ContinuanceToken != nullptr)
		{
			if (bAutoLinkAccount)
			{
				EOS_Auth_LinkAccountOptions LinkAccountOptions = {};
				LinkAccountOptions.ApiVersion = 1;
				LinkAccountOptions.ContinuanceToken = Data->ContinuanceToken;
				LinkAccountOptions.LinkAccountFlags = LinkAccountFlags;
				UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_LINKACCOUNT_API_LATEST, 1);

				EOS_Async(EOS_Auth_LinkAccount, AuthHandle, LinkAccountOptions,
				[Promise = MoveTemp(Promise)](const EOS_Auth_LinkAccountCallbackInfo* Data) mutable -> void
				{
					UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LoginEASImpl] EOS_Auth_LinkAccount Result: [%ls]", *LexToString(Data->ResultCode));

					if (Data->ResultCode == EOS_EResult::EOS_Success)
					{
						Promise.SetValue(TDefaultErrorResult<FAuthLoginEASImpl>(FAuthLoginEASImpl::Result{ Data->LocalUserId }));
					}
					else
					{
						Promise.SetValue(TDefaultErrorResult<FAuthLoginEASImpl>(Errors::FromEOSResult(Data->ResultCode)));
					}
				});
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LoginEASImpl] Saving link account continuance token for manual link account usage.");
				if (FUserScopedData* UserData = GetOrCreateUserScopedData(PlatformUserId))
				{
					UserData->LastLoginContinuationId = FLoginContinuationId(EOnlineServices::Epic, NextLoginContinuationId++);
					UserData->LoginContinuations.Add(FLoginContinuationData{UserData->LastLoginContinuationId, Data->ContinuanceToken, LinkAccountFlags});
				}
				else
				{
					UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::LoginEASImpl] Failed to access user scoped data.");
				}

				Promise.SetValue(TDefaultErrorResult<FAuthLoginEASImpl>(Errors::FromEOSResult(Data->ResultCode)));
			}
		}
		else
		{
			FOnlineError ResolvedError = Errors::FromEOSResult(Data->ResultCode);

			const bool bShouldRemoveCachedToken =
				Data->ResultCode == EOS_EResult::EOS_InvalidAuth ||
				Data->ResultCode == EOS_EResult::EOS_AccessDenied ||
				Data->ResultCode == EOS_EResult::EOS_Auth_InvalidToken;

			// Remove persistent auth credentials when they are found to be invalid.
			if (bIsPersistentAuthLogin && bShouldRemoveCachedToken)
			{
				EOS_Auth_DeletePersistentAuthOptions DeletePersistentAuthOptions = {};
				DeletePersistentAuthOptions.ApiVersion = 2;
				UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_DELETEPERSISTENTAUTH_API_LATEST, 2);

				EOS_Async(EOS_Auth_DeletePersistentAuth, AuthHandle, DeletePersistentAuthOptions,
				[ResolvedError = MoveTemp(ResolvedError), Promise = MoveTemp(Promise)](const EOS_Auth_DeletePersistentAuthCallbackInfo* Data) mutable -> void
				{
					if (Data->ResultCode != EOS_EResult::EOS_Success)
					{
						FOnlineError DeletePersistentAuthError(Errors::FromEOSResult(Data->ResultCode));
						UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::LoginEASImpl] Failure: DeletePersistentAuthResult %ls", *DeletePersistentAuthError.GetLogString());
					}

					Promise.SetValue(TDefaultErrorResult<FAuthLoginEASImpl>(MoveTemp(ResolvedError)));
				});
			}
			else
			{
				Promise.SetValue(TDefaultErrorResult<FAuthLoginEASImpl>(MoveTemp(ResolvedError)));
			}
		}
	});

	return Future;
}

TFuture<TDefaultErrorResult<FAuthLogoutEASImpl>> FAuthEOSGS::LogoutEASImpl(const FAuthLogoutEASImpl::Params& LogoutParams)
{
	TPromise<TDefaultErrorResult<FAuthLogoutEASImpl>> Promise;
	TFuture<TDefaultErrorResult<FAuthLogoutEASImpl>> Future = Promise.GetFuture();

	EOS_Auth_LogoutOptions LogoutOptions = {};
	LogoutOptions.ApiVersion = 1;
	LogoutOptions.LocalUserId = LogoutParams.EpicAccountId;
	UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_LOGOUT_API_LATEST, 1);

	EOS_Async(EOS_Auth_Logout, AuthHandle, LogoutOptions,
	[Promise = MoveTemp(Promise)](const EOS_Auth_LogoutCallbackInfo* Data) mutable -> void
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LogoutEASImpl] EOS_Auth_Logout Result: [%ls]", *LexToString(Data->ResultCode));
			Promise.SetValue(TDefaultErrorResult<FAuthLogoutEASImpl>(FAuthLogoutEASImpl::Result{}));
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::LogoutEASImpl] EOS_Auth_Logout Result: [%ls]", *LexToString(Data->ResultCode));
			Promise.SetValue(TDefaultErrorResult<FAuthLogoutEASImpl>(Errors::FromEOSResult(Data->ResultCode)));
		}
	});

	return Future;
}

TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> FAuthEOSGS::GetExternalAuthTokenImpl(const FAuthGetExternalAuthTokenImpl::Params& Params)
{
	if (CVarEnableExternalAuthUserAuthToken.GetValueOnGameThread())
	{
		EOS_Auth_CopyUserAuthTokenOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST, 1);

		EOS_Auth_Token* Token = nullptr;
		const EOS_EResult Result = EOS_Auth_CopyUserAuthToken(AuthHandle, &Options, Params.EpicAccountId, &Token);
		if (Result == EOS_EResult::EOS_Success)
		{
			ON_SCOPE_EXIT
			{
				EOS_Auth_Token_Release(Token);
			};

			FExternalAuthToken ExternalAuthToken;
			ExternalAuthToken.Type = ExternalLoginType::Epic;
			ExternalAuthToken.Data = UTF8_TO_TCHAR(Token->AccessToken);
			return TDefaultErrorResult<FAuthGetExternalAuthTokenImpl>(FAuthGetExternalAuthTokenImpl::Result{ MoveTemp(ExternalAuthToken) });
		}
		else
		{
			return TDefaultErrorResult<FAuthGetExternalAuthTokenImpl>(Errors::FromEOSResult(Result));
		}
	}
	else
	{
		EOS_Auth_CopyIdTokenOptions Options = {};
		Options.ApiVersion = 1;
		Options.AccountId = Params.EpicAccountId;
		UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_COPYIDTOKEN_API_LATEST, 1);

		EOS_Auth_IdToken* IdToken = nullptr;
		const EOS_EResult Result = EOS_Auth_CopyIdToken(AuthHandle, &Options, &IdToken);
		if (Result == EOS_EResult::EOS_Success)
		{
			ON_SCOPE_EXIT
			{
				EOS_Auth_IdToken_Release(IdToken);
			};

			FExternalAuthToken ExternalAuthToken;
			ExternalAuthToken.Type = ExternalLoginType::EpicIdToken;
			ExternalAuthToken.Data = UTF8_TO_TCHAR(IdToken->JsonWebToken);
			return TDefaultErrorResult<FAuthGetExternalAuthTokenImpl>(FAuthGetExternalAuthTokenImpl::Result{ MoveTemp(ExternalAuthToken) });
		}
		else
		{
			return TDefaultErrorResult<FAuthGetExternalAuthTokenImpl>(Errors::FromEOSResult(Result));
		}
	}
}

TFuture<TDefaultErrorResult<FAuthLoginConnectImpl>> FAuthEOSGS::LoginConnectImpl(const FAuthLoginConnectImpl::Params& LoginParams)
{
	TDefaultErrorResultInternal<FEOSConnectLoginOptions> LoginOptionsResult = FEOSConnectLoginOptions::Create(LoginParams.PlatformUserId, LoginParams.ExternalAuthToken);
	if (LoginOptionsResult.IsError())
	{
		return MakeFulfilledPromise<TDefaultErrorResult<FAuthLoginConnectImpl>>(MoveTemp(LoginOptionsResult.GetErrorValue())).GetFuture();
	}

	TPromise<TDefaultErrorResult<FAuthLoginConnectImpl>> Promise;
	TFuture<TDefaultErrorResult<FAuthLoginConnectImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Connect_Login, ConnectHandle, MoveTemp(LoginOptionsResult.GetOkValue()),
	[ConnectHandle = ConnectHandle, Promise = MoveTemp(Promise)](const EOS_Connect_LoginCallbackInfo* Data) mutable -> void
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LoginConnect] EOS_Connect_Login Result: [%ls]", *LexToString(Data->ResultCode));

		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			Promise.SetValue(TDefaultErrorResult<FAuthLoginConnectImpl>(FAuthLoginConnectImpl::Result{ Data->LocalUserId }));
		}
		else if (Data->ResultCode == EOS_EResult::EOS_InvalidUser && Data->ContinuanceToken != nullptr)
		{
			EOS_Connect_CreateUserOptions ConnectCreateUserOptions = { };
			ConnectCreateUserOptions.ApiVersion = 1;
			ConnectCreateUserOptions.ContinuanceToken = Data->ContinuanceToken;
			UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_CREATEUSER_API_LATEST, 1);

			EOS_Async(EOS_Connect_CreateUser, ConnectHandle, ConnectCreateUserOptions,
			[Promise = MoveTemp(Promise)](const EOS_Connect_CreateUserCallbackInfo* Data) mutable -> void
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LoginConnect] EOS_Connect_CreateUser Result: [%ls]", *LexToString(Data->ResultCode));

				if (Data->ResultCode == EOS_EResult::EOS_Success)
				{
					Promise.SetValue(TDefaultErrorResult<FAuthLoginConnectImpl>(FAuthLoginConnectImpl::Result{ Data->LocalUserId }));
				}
				else
				{
					Promise.SetValue(TDefaultErrorResult<FAuthLoginConnectImpl>(Errors::FromEOSResult(Data->ResultCode)));
				}
			});
		}
		else
		{
			Promise.SetValue(TDefaultErrorResult<FAuthLoginConnectImpl>(Errors::FromEOSResult(Data->ResultCode)));
		}
	});

	return Future;
}

ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FAuthLogoutConnectImpl>> FAuthEOSGS::LogoutConnectImpl(const FAuthLogoutConnectImpl::Params& LogoutParams)
{
	TPromise<TDefaultErrorResult<FAuthLogoutConnectImpl>> Promise;
	TFuture<TDefaultErrorResult<FAuthLogoutConnectImpl>> Future = Promise.GetFuture();

	EOS_Connect_LogoutOptions LogoutOptions = {};
	LogoutOptions.ApiVersion = 1; 
	LogoutOptions.LocalUserId = LogoutParams.ProductUserId;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_LOGOUT_API_LATEST, 1);

	EOS_Async(EOS_Connect_Logout, ConnectHandle, LogoutOptions,
		[Promise = MoveTemp(Promise)](const EOS_Connect_LogoutCallbackInfo* Data) mutable -> void
		{
			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOSGS::LogoutConnectImpl] EOS_Connect_Logout Result: [%ls]", *LexToString(Data->ResultCode));
				Promise.SetValue(TDefaultErrorResult<FAuthLogoutConnectImpl>(FAuthLogoutConnectImpl::Result{}));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::LogoutConnectImpl] EOS_Connect_Logout Result: [%ls]", *LexToString(Data->ResultCode));
				Promise.SetValue(TDefaultErrorResult<FAuthLogoutConnectImpl>(Errors::FromEOSResult(Data->ResultCode)));
			}
		});

	return Future;
}

TOnlineAsyncOpHandle<FAuthConnectLoginRecoveryImpl> FAuthEOSGS::ConnectLoginRecoveryImplOp(FAuthConnectLoginRecoveryImpl::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthConnectLoginRecoveryImpl> Op = GetOp<FAuthConnectLoginRecoveryImpl>(MoveTemp(Params));

	// Step 1: Setup operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthConnectLoginRecoveryImpl>& InAsyncOp)
	{
		const FAuthConnectLoginRecoveryImpl::Params& Params = InAsyncOp.GetParams();
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalUserId);
		if (!AccountInfoEOS)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		if (AccountInfoEOS->LoginStatus != ELoginStatus::LoggedInReducedFunctionality)
		{
			InAsyncOp.SetError(Errors::Unknown());
			return;
		}

		if (!ensure(AccountInfoEOS->EpicAccountId))
		{
			InAsyncOp.SetError(Errors::Unknown());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(AccountInfoKeyName, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Acquire EAS external auth token.
	.Then([this](TOnlineAsyncOp<FAuthConnectLoginRecoveryImpl>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> AuthTokenResult = GetExternalAuthTokenImpl(FAuthGetExternalAuthTokenImpl::Params{ AccountInfoEOS->EpicAccountId });
		if (AuthTokenResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::ConnectLoginRecoveryImplOp] Failure: GetExternalAuthTokenImpl %ls", *AuthTokenResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(AuthTokenResult.GetErrorValue()));

			// Reinitialize recovery timer.
			InitializeConnectLoginRecoveryTimer(AccountInfoEOS);
			return FAuthLoginConnectImpl::Params{};
		}

		return FAuthLoginConnectImpl::Params{ AccountInfoEOS->PlatformUserId, MoveTemp(AuthTokenResult.GetOkValue().Token) };
	})
	// Step 3: Refresh connect login.
	.Then([this](TOnlineAsyncOp<FAuthConnectLoginRecoveryImpl>& InAsyncOp, FAuthLoginConnectImpl::Params&& LoginConnectParams)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		// Attempt connect login.
		LoginConnectImpl(LoginConnectParams)
		.Next([this, AccountInfoEOS, Op = InAsyncOp.AsShared(), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLoginConnectImpl>&& LoginResult) mutable -> void
		{
			if (LoginResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::ConnectLoginRecoveryImplOp] Failure: LoginConnectImpl %ls", *LoginResult.GetErrorValue().GetLogString());
				Op->SetError(MoveTemp(LoginResult.GetErrorValue()));

				// Reinitialize recovery timer.
				InitializeConnectLoginRecoveryTimer(AccountInfoEOS);
				Promise.EmplaceValue();
			}
			else
			{
				// Successful login.
				Promise.EmplaceValue();
			}
		});

		return Future;
	})
	// Step 4: Update cache and notify.
	.Then([this](TOnlineAsyncOp<FAuthConnectLoginRecoveryImpl>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		UE_LOGF(LogOnlineServices, Log, "[FAuthEOSGS::ConnectLoginRecoveryImplOp] User has restored online capability [%ls]", *ToLogString(AccountInfoEOS->AccountId));
		AccountInfoEOS->LoginStatus = ELoginStatus::LoggedIn;
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });

		InAsyncOp.SetResult(FAuthConnectLoginRecoveryImpl::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthHandleConnectLoginStatusChangedImpl> FAuthEOSGS::HandleConnectLoginStatusChangedImplOp(FAuthHandleConnectLoginStatusChangedImpl::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthHandleConnectLoginStatusChangedImpl> Op = GetOp<FAuthHandleConnectLoginStatusChangedImpl>(MoveTemp(Params));

	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthHandleConnectLoginStatusChangedImpl>& InAsyncOp)
	{
		const FAuthHandleConnectLoginStatusChangedImpl::Params& Params = InAsyncOp.GetParams();
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalUserId);
		if (!AccountInfoEOS)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(AccountInfoKeyName, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Update status and notify.
	.Then([this](TOnlineAsyncOp<FAuthHandleConnectLoginStatusChangedImpl>& InAsyncOp)
	{
		const FAuthHandleConnectLoginStatusChangedImpl::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		// Check if the user was kicked offline.
		if (Params.CurrentStatus == EOS_ELoginStatus::EOS_LS_NotLoggedIn)
		{
			// Check if user can be brought back online automatically.
			if (AccountInfoEOS->EpicAccountId)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::HandleConnectLoginStatusChangedImplOp] User has reduced online capability [%ls]", *ToLogString(AccountInfoEOS->AccountId));
				AccountInfoEOS->LoginStatus = ELoginStatus::LoggedInReducedFunctionality;
				OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });

				// Auth has now entered degraded state for the user. Set timer to periodically attempt to reestablish full auth connection.
				InitializeConnectLoginRecoveryTimer(AccountInfoEOS);
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::HandleConnectLoginStatusChangedImplOp] User is offline [%ls]", *ToLogString(AccountInfoEOS->AccountId));
				AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;
				OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });
				AccountInfoRegistryEOS.Unregister(AccountInfoEOS->AccountId);
			}
		}

		InAsyncOp.SetResult(FAuthHandleConnectLoginStatusChangedImpl::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthHandleConnectAuthNotifyExpirationImpl> FAuthEOSGS::HandleConnectAuthNotifyExpirationImplOp(FAuthHandleConnectAuthNotifyExpirationImpl::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthHandleConnectAuthNotifyExpirationImpl> Op = GetOp<FAuthHandleConnectAuthNotifyExpirationImpl>(MoveTemp(Params));

	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthHandleConnectAuthNotifyExpirationImpl>& InAsyncOp)
	{
		const FAuthHandleConnectAuthNotifyExpirationImpl::Params& Params = InAsyncOp.GetParams();
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalUserId);
		if (!AccountInfoEOS)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		if (AccountInfoEOS->LoginStatus != ELoginStatus::LoggedIn)
		{
			InAsyncOp.SetError(Errors::NotLoggedIn());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(AccountInfoKeyName, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Automatically start recovery or notify user to start recovery.
	.Then([this](TOnlineAsyncOp<FAuthHandleConnectAuthNotifyExpirationImpl>& InAsyncOp)
	{
		const FAuthHandleConnectAuthNotifyExpirationImpl::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		// When EAS is logged in use it to refresh the connect login.
		if (AccountInfoEOS->EpicAccountId)
		{
			// Auth has now entered degraded state for the user. Set timer to periodically attempt to reestablish full auth connection.
			InitializeConnectLoginRecoveryTimer(AccountInfoEOS);
		}
		else
		{
			// No EAS login, notify user code to refresh connect login using external auth.
			OnAuthPendingAuthExpirationEvent.Broadcast(FAuthPendingAuthExpiration{ AccountInfoEOS });
		}

		InAsyncOp.SetResult(FAuthHandleConnectAuthNotifyExpirationImpl::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthHandleEASLoginStatusChangedImpl> FAuthEOSGS::HandleEASLoginStatusChangedImplOp(FAuthHandleEASLoginStatusChangedImpl::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthHandleEASLoginStatusChangedImpl> Op = GetOp<FAuthHandleEASLoginStatusChangedImpl>(MoveTemp(Params));

	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthHandleEASLoginStatusChangedImpl>& InAsyncOp)
	{
		const FAuthHandleEASLoginStatusChangedImpl::Params& Params = InAsyncOp.GetParams();
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalUserId);
		if (!AccountInfoEOS)
		{
			InAsyncOp.SetError(Errors::InvalidUser());
			return;
		}

		if (AccountInfoEOS->LoginStatus != ELoginStatus::LoggedIn)
		{
			InAsyncOp.SetError(Errors::NotLoggedIn());
			return;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(AccountInfoKeyName, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Update status and notify.
	.Then([this](TOnlineAsyncOp<FAuthHandleEASLoginStatusChangedImpl>& InAsyncOp)
	{
		const FAuthHandleEASLoginStatusChangedImpl::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, AccountInfoKeyName);

		// Check if the user was kicked offline.
		if (Params.CurrentStatus == EOS_ELoginStatus::EOS_LS_NotLoggedIn)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FAuthEOSGS::HandleEASLoginStatusChangedImplOp] User is now offline [%ls]", *ToLogString(AccountInfoEOS->AccountId));
			AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;
			OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });
			AccountInfoRegistryEOS.Unregister(AccountInfoEOS->AccountId);
		}

		InAsyncOp.SetResult(FAuthHandleEASLoginStatusChangedImpl::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

void FAuthEOSGS::RegisterHandlers()
{
	{
		// Register for EOS connect connection status updates.
		const int ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_ADDNOTIFYLOGINSTATUSCHANGED_API_LATEST, ApiVersion);
		OnConnectLoginStatusChangedEOSEventRegistration = EOS_RegisterComponentEventHandler(
			this,
			ConnectHandle,
			ApiVersion,
			&EOS_Connect_AddNotifyLoginStatusChanged,
			&EOS_Connect_RemoveNotifyLoginStatusChanged,
			&FAuthEOSGS::OnConnectLoginStatusChanged);
	}

	{
		// Notification of a pending external token expiration ~10 minutes.
		const int ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_ADDNOTIFYAUTHEXPIRATION_API_LATEST, ApiVersion);
		OnConnectAuthNotifyExpirationEOSEventRegistration = EOS_RegisterComponentEventHandler(
			this,
			ConnectHandle,
			ApiVersion,
			&EOS_Connect_AddNotifyAuthExpiration,
			&EOS_Connect_RemoveNotifyAuthExpiration,
			&FAuthEOSGS::OnConnectAuthNotifyExpiration);
	}

	{
		// Register for EAS connection status updates.
		const int ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_ADDNOTIFYLOGINSTATUSCHANGED_API_LATEST, ApiVersion);
		OnAuthLoginStatusChangedEOSEventRegistration = EOS_RegisterComponentEventHandler(
			this,
			AuthHandle,
			ApiVersion,
			&EOS_Auth_AddNotifyLoginStatusChanged,
			&EOS_Auth_RemoveNotifyLoginStatusChanged,
			&FAuthEOSGS::OnEASLoginStatusChanged);
	}
}

void FAuthEOSGS::UnregisterHandlers()
{
	OnConnectLoginStatusChangedEOSEventRegistration = nullptr;
	OnConnectAuthNotifyExpirationEOSEventRegistration = nullptr;
	OnAuthLoginStatusChangedEOSEventRegistration = nullptr;
}

void FAuthEOSGS::OnConnectLoginStatusChanged(const EOS_Connect_LoginStatusChangedCallbackInfo* Data)
{
	HandleConnectLoginStatusChangedImplOp(FAuthHandleConnectLoginStatusChangedImpl::Params{ Data->LocalUserId, Data->PreviousStatus, Data->CurrentStatus });
}

void FAuthEOSGS::OnConnectAuthNotifyExpiration(const EOS_Connect_AuthExpirationCallbackInfo* Data)
{
	HandleConnectAuthNotifyExpirationImplOp(FAuthHandleConnectAuthNotifyExpirationImpl::Params{ Data->LocalUserId });
}

void FAuthEOSGS::OnEASLoginStatusChanged(const EOS_Auth_LoginStatusChangedCallbackInfo* Data)
{
	HandleEASLoginStatusChangedImplOp(FAuthHandleEASLoginStatusChangedImpl::Params{ Data->LocalUserId, Data->PrevStatus, Data->CurrentStatus });
}

#if !UE_BUILD_SHIPPING
void FAuthEOSGS::CheckMetadata()
{
	// Metadata sanity check.
	ToLogString(FAuthLoginEASImpl::Params());
	ToLogString(FAuthLoginEASImpl::Result());
	ToLogString(FAuthLogoutEASImpl::Params());
	ToLogString(FAuthLogoutEASImpl::Result());
	ToLogString(FAuthGetExternalAuthTokenImpl::Params());
	ToLogString(FAuthGetExternalAuthTokenImpl::Result());
	ToLogString(FAuthLoginConnectImpl::Params());
	ToLogString(FAuthLoginConnectImpl::Result());
	ToLogString(FAuthConnectLoginRecoveryImpl::Params());
	ToLogString(FAuthConnectLoginRecoveryImpl::Result());
	ToLogString(FAuthHandleConnectLoginStatusChangedImpl::Params());
	ToLogString(FAuthHandleConnectLoginStatusChangedImpl::Result());
	ToLogString(FAuthHandleConnectAuthNotifyExpirationImpl::Params());
	ToLogString(FAuthHandleConnectAuthNotifyExpirationImpl::Result());
	ToLogString(FAuthHandleEASLoginStatusChangedImpl::Params());
	ToLogString(FAuthHandleEASLoginStatusChangedImpl::Result());
	ToLogString(FAccountInfoEOS());
	Meta::VisitFields(FAuthLoginEASImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthLoginEASImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthLogoutEASImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthLogoutEASImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthGetExternalAuthTokenImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthGetExternalAuthTokenImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthLoginConnectImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthLoginConnectImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthConnectLoginRecoveryImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthConnectLoginRecoveryImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleConnectLoginStatusChangedImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleConnectLoginStatusChangedImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleConnectAuthNotifyExpirationImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleConnectAuthNotifyExpirationImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleEASLoginStatusChangedImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FAuthHandleEASLoginStatusChangedImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
}
#endif

const FAccountInfoRegistry& FAuthEOSGS::GetAccountInfoRegistry() const
{
	return AccountInfoRegistryEOS;
}

void FAuthEOSGS::InitializeConnectLoginRecoveryTimer(const TSharedRef<FAccountInfoEOS>& AccountInfoEOS)
{
	FAuthEOSGSLoginRecoveryConfig AuthEOSGSLoginRecoveryConfig;
	LoadConfig(AuthEOSGSLoginRecoveryConfig, TEXT("LoginRecovery"));
	const float DelayTime = AuthEOSGSLoginRecoveryConfig.Interval + AuthEOSGSLoginRecoveryConfig.Jitter * FMath::RandRange(0.f, 1.f);

	if (AccountInfoEOS->RestoreLoginTimer.IsValid())
	{
		FTSBackgroundableTicker::GetCoreTicker().RemoveTicker(AccountInfoEOS->RestoreLoginTimer);
	}

	AccountInfoEOS->RestoreLoginTimer = FTSBackgroundableTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([this, WeakThis = AsWeak(), EpicAccountId = AccountInfoEOS->EpicAccountId](float) -> bool
		{
			if (WeakThis.IsValid())
			{
				ConnectLoginRecoveryImplOp(FAuthConnectLoginRecoveryImpl::Params { EpicAccountId });
			}

			// One-shot timer.
			return false;
		}),
		DelayTime);
}

FAccountId FAuthEOSGS::CreateAccountId(const EOS_ProductUserId ProductUserId)
{
	return FOnlineAccountIdRegistryEOSGS::GetRegistered(EOnlineServices::Epic).FindOrAddAccountId(ProductUserId);
}

FAuthEOSGS::FUserScopedData* FAuthEOSGS::GetUserScopedData(FPlatformUserId PlatformUserId)
{
	const int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
	return UserScopedData.IsValidIndex(UserIndex) ? &UserScopedData[UserIndex] : nullptr;
}

const FAuthEOSGS::FUserScopedData* FAuthEOSGS::GetUserScopedData(FPlatformUserId PlatformUserId) const
{
	const int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
	return UserScopedData.IsValidIndex(UserIndex) ? &UserScopedData[UserIndex] : nullptr;
}

FAuthEOSGS::FUserScopedData* FAuthEOSGS::GetOrCreateUserScopedData(FPlatformUserId PlatformUserId)
{
	const int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
	if (UserIndex == INDEX_NONE)
	{
		return nullptr;
	}

	if (!UserScopedData.IsValidIndex(UserIndex))
	{
		UserScopedData.Insert(UserIndex, FUserScopedData());
	}

	return &UserScopedData[UserIndex];
}

/* UE::Online */ }
