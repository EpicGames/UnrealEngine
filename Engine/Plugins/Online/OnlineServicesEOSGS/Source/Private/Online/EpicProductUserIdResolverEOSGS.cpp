// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/EpicProductUserIdResolverEOSGS.h"

#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

namespace UE::Online {

FEpicProductUserIdResolverEOSGS::FEpicProductUserIdResolverEOSGS(FOnlineServicesEOSGS& InServices)
: TOnlineComponent(TEXT("EpicProductUserIdResolver"), InServices)
{
}

TFuture<FAccountId> FEpicProductUserIdResolverEOSGS::ResolveAccountId(const FAccountId LocalAccountId, const EOS_ProductUserId ProductUserId)
{
	const FAccountId AccountId = FOnlineAccountIdRegistryEOSGS::GetRegistered(EOnlineServices::Epic).FindOrAddAccountId(ProductUserId);
	
	return MakeFulfilledPromise<FAccountId>(AccountId).GetFuture();
}

TFuture<TArray<FAccountId>> FEpicProductUserIdResolverEOSGS::ResolveAccountIds(const FAccountId LocalAccountId, const TArray<EOS_ProductUserId>& ProductUserIds)
{
	TArray<FAccountId> AccountIds;
	AccountIds.Reserve(ProductUserIds.Num());
	for (const EOS_ProductUserId ProductUserId : ProductUserIds)
	{
		const FAccountId AccountId = FOnlineAccountIdRegistryEOSGS::GetRegistered(EOnlineServices::Epic).FindOrAddAccountId(ProductUserId);
		AccountIds.Emplace(AccountId);
	}
	return MakeFulfilledPromise<TArray<FAccountId>>(MoveTemp(AccountIds)).GetFuture();
}

/* UE::Online */ }
