// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/ExternalUIEOSGS.h"

#include "IEOSSDKManager.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_ui.h"

namespace UE::Online {

FExternalUIEOSGS::FExternalUIEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

FExternalUIEOSGS::~FExternalUIEOSGS() = default;

void FExternalUIEOSGS::Initialize()
{
	Super::Initialize();

	IEOSSDKManager* SDKManager = IEOSSDKManager::Get();
	check(SDKManager);

	UIInterfaceHandle = EOS_Platform_GetUIInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(UIInterfaceHandle != nullptr);

	RegisterEventHandlers();
}

void FExternalUIEOSGS::PreShutdown()
{
	Super::PreShutdown();

	UnregisterEventHandlers();

	UIInterfaceHandle = nullptr;
}

void FExternalUIEOSGS::RegisterEventHandlers()
{
	// This delegate would cause a crash when running a dedicated server
	if (!IsRunningDedicatedServer())
	{
		OnDisplaySettingsUpdated = EOS_RegisterComponentEventHandler(
			this,
			UIInterfaceHandle,
			EOS_UI_ADDNOTIFYDISPLAYSETTINGSUPDATED_API_LATEST,
			&EOS_UI_AddNotifyDisplaySettingsUpdated,
			&EOS_UI_RemoveNotifyDisplaySettingsUpdated,
			&FExternalUIEOSGS::HandleDisplaySettingsUpdated);
		UE_EOS_CHECK_API_MISMATCH(EOS_UI_ADDNOTIFYDISPLAYSETTINGSUPDATED_API_LATEST, 1);
	}
}

void FExternalUIEOSGS::UnregisterEventHandlers()
{
	OnDisplaySettingsUpdated = nullptr;
}

void FExternalUIEOSGS::HandleDisplaySettingsUpdated(const EOS_UI_OnDisplaySettingsUpdatedCallbackInfo* Data)
{
	check(Data);
	FExternalUIProcessDisplaySettingsUpdatedImp::Params Params;
	Params.bIsVisible = Data->bIsVisible == EOS_TRUE;
	Params.bIsExclusiveInput = Data->bIsExclusiveInput == EOS_TRUE;
	ProcessDisplaySettingsUpdatedImplOp(MoveTemp(Params));
}

TOnlineAsyncOpHandle<FExternalUIProcessDisplaySettingsUpdatedImp> FExternalUIEOSGS::ProcessDisplaySettingsUpdatedImplOp(FExternalUIProcessDisplaySettingsUpdatedImp::Params&& Params)
{
	TOnlineAsyncOpRef<FExternalUIProcessDisplaySettingsUpdatedImp> Op = GetOp<FExternalUIProcessDisplaySettingsUpdatedImp>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FExternalUIProcessDisplaySettingsUpdatedImp>& InAsyncOp)
	{
		const FExternalUIProcessDisplaySettingsUpdatedImp::Params& Params = InAsyncOp.GetParams();

		UE_LOGF(LogOnlineServices, Log, "[FExternalUIEOSGS::ProcessDisplaySettingsUpdatedImplOp] Display settings changed notification received. bIsVisible: %ls, bIsExclusiveInput: %ls", *::LexToString(Params.bIsVisible), *::LexToString(Params.bIsExclusiveInput));

		if (Params.bIsExclusiveInput != bLastReportedExclusiveInput)
		{
			bLastReportedExclusiveInput = Params.bIsExclusiveInput;
			OnExternalUIStatusChangedEvent.Broadcast(FExternalUIStatusChanged{ Params.bIsExclusiveInput });
		}

		InAsyncOp.SetResult(FExternalUIProcessDisplaySettingsUpdatedImp::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

/* UE::Online */ }
