// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSEOSAuthLoginOptions.h"

#include "EOSShared.h"
#include "IOS/IOSAppDelegate.h"

#import <AuthenticationServices/AuthenticationServices.h>

@interface EOSAuthPresentationContext: NSObject <ASWebAuthenticationPresentationContextProviding>
@end

@implementation EOSAuthPresentationContext

- (ASPresentationAnchor)presentationAnchorForWebAuthenticationSession:(ASWebAuthenticationSession *)session API_AVAILABLE(ios(13.0))
{
    return [[IOSAppDelegate GetDelegate] window];
}

@end

namespace UE::Online
{

FIOSEOSAuthLoginOptions::FIOSEOSAuthLoginOptions(FIOSEOSAuthLoginOptions&& Other)
{
    *this = MoveTemp(Other);
}

FIOSEOSAuthLoginOptions& FIOSEOSAuthLoginOptions::operator=(FIOSEOSAuthLoginOptions&& Other)
{
    if (CredentialsOptions.PresentationContextProviding)
	{
        CFRelease((CFTypeRef*)CredentialsOptions.PresentationContextProviding);
        CredentialsOptions = {};
    }
	
    FEOSAuthLoginOptionsCommon::operator=(MoveTemp(Other));

    if (Other.CredentialsData.SystemAuthCredentialsOptions)
	{
		CredentialsOptions.ApiVersion = Other.CredentialsOptions.ApiVersion;
		CredentialsOptions.PresentationContextProviding = Other.CredentialsOptions.PresentationContextProviding;
		CredentialsData.SystemAuthCredentialsOptions = &CredentialsOptions;
	}
	else
	{
		CredentialsData.SystemAuthCredentialsOptions = nullptr;
	}
	return *this;
}

bool FIOSEOSAuthLoginOptions::InitSystemAuthCredentialOptions(FIOSEOSAuthLoginOptions& Options)
{
	UE_EOS_CHECK_API_MISMATCH(EOS_IOS_AUTH_CREDENTIALSOPTIONS_API_LATEST, 2);
    Options.CredentialsOptions.ApiVersion = 1;
	Options.CredentialsOptions.PresentationContextProviding = (void*)CFBridgingRetain([[EOSAuthPresentationContext alloc] init]);
	Options.CredentialsData.SystemAuthCredentialsOptions = &Options.CredentialsOptions;
    return true;
}

}
