// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/EOSAuthLoginOptionsCommon.h"

#include "eos_ios.h"

namespace UE::Online
{

class FIOSEOSAuthLoginOptions : public FEOSAuthLoginOptionsCommon
{
public:
	FIOSEOSAuthLoginOptions(FIOSEOSAuthLoginOptions&&);
	FIOSEOSAuthLoginOptions& operator=(FIOSEOSAuthLoginOptions&&);

protected:
	FIOSEOSAuthLoginOptions() = default;

    static bool InitSystemAuthCredentialOptions(FIOSEOSAuthLoginOptions& Options);

private:
    EOS_IOS_Auth_CredentialsOptions CredentialsOptions = {};
};

using FPlatformEOSAuthLoginOptions = FIOSEOSAuthLoginOptions;
}
