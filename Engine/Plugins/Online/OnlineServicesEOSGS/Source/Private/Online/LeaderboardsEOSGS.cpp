// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LeaderboardsEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_leaderboards.h"

namespace UE::Online {

#define EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH 256 + 1 // 256 plus null terminator
#define EOS_LEADERBOARD_MAX_ENTRIES 1000

FLeaderboardsEOSGS::FLeaderboardsEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

void FLeaderboardsEOSGS::Initialize()
{
	Super::Initialize();

	LeaderboardsHandle = EOS_Platform_GetLeaderboardsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(LeaderboardsHandle);
}

TOnlineAsyncOpHandle<FReadEntriesForUsers> FLeaderboardsEOSGS::ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesForUsers> Op = GetJoinableOp<FReadEntriesForUsers>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FReadEntriesForUsers>& InAsyncOp)
		{
			const FReadEntriesForUsers::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FReadEntriesForUsers>& InAsyncOp, TPromise<const EOS_Leaderboards_OnQueryLeaderboardUserScoresCompleteCallbackInfo*>&& Promise)
		{
			const FReadEntriesForUsers::Params& Params = InAsyncOp.GetParams();

			TArray<EOS_ProductUserId> ProductUserIds;
			ProductUserIds.AddZeroed(Params.AccountIds.Num());
			for (const FAccountId& AccountId : Params.AccountIds)
			{
				ProductUserIds.Emplace(GetProductUserIdChecked(AccountId));
			}

			EOS_Leaderboards_UserScoresQueryStatInfo StatInfo;
			StatInfo.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_USERSCORESQUERYSTATINFO_API_LATEST, 1);

			char StatNameANSI[EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH];
			FCStringAnsi::Strncpy(StatNameANSI, TCHAR_TO_UTF8(*Params.BoardName), EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH);

			StatInfo.StatName = StatNameANSI;
			StatInfo.Aggregation = EOS_ELeaderboardAggregation::EOS_LA_Latest; // TODO: Use Stats definitions

			EOS_Leaderboards_QueryLeaderboardUserScoresOptions Options = { };
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_QUERYLEADERBOARDUSERSCORES_API_LATEST, 2);
			Options.UserIds = ProductUserIds.GetData();
			Options.UserIdsCount = ProductUserIds.Num();
			Options.StatInfo = &StatInfo;
			Options.StatInfoCount = 1;
			Options.StartTime = EOS_LEADERBOARDS_TIME_UNDEFINED;
			Options.EndTime = EOS_LEADERBOARDS_TIME_UNDEFINED;
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

			EOS_Async(EOS_Leaderboards_QueryLeaderboardUserScores, LeaderboardsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FReadEntriesForUsers>& InAsyncOp, const EOS_Leaderboards_OnQueryLeaderboardUserScoresCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Leaderboards_IngestStat failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			const FReadEntriesForUsers::Params& Params = InAsyncOp.GetParams();

			char StatNameANSI[EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH];
			FCStringAnsi::Strncpy(StatNameANSI, TCHAR_TO_UTF8(*Params.BoardName), EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH);

			FReadEntriesForUsers::Result Result;

			for (const FAccountId& AccountId : Params.AccountIds)
			{
				EOS_Leaderboards_CopyLeaderboardUserScoreByUserIdOptions UserCopyOptions = { };
				UserCopyOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDUSERSCOREBYUSERID_API_LATEST, 1);

				UserCopyOptions.UserId = GetProductUserIdChecked(AccountId);
				UserCopyOptions.StatName = StatNameANSI;

				EOS_Leaderboards_LeaderboardUserScore* LeaderboardUserScore = nullptr;
				EOS_EResult CopyResult = EOS_Leaderboards_CopyLeaderboardUserScoreByUserId(LeaderboardsHandle, &UserCopyOptions, &LeaderboardUserScore);

				if (CopyResult != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, VeryVerbose, "Value not found for leaderboard %ls.", *Params.BoardName);
					continue;
				}

				FLeaderboardEntry& Entry = Result.Entries.Emplace_GetRef();
				Entry.Rank = UE_LEADERBOARD_RANK_UNKNOWN;
				Entry.AccountId = AccountId;
				Entry.Score = LeaderboardUserScore->Score;

				EOS_Leaderboards_LeaderboardUserScore_Release(LeaderboardUserScore);
			}

			InAsyncOp.SetResult(MoveTemp(Result));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

namespace Private
{

void QueryLeaderboardsEOS(EOS_HLeaderboards LeaderboardsHandle, const FAccountId& LocalAccountId, const FString& BoardName, TPromise<const EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo*>&& Promise)
{
	EOS_Leaderboards_QueryLeaderboardRanksOptions Options;
	Options.LocalUserId = GetProductUserIdChecked(LocalAccountId);
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_QUERYLEADERBOARDRANKS_API_LATEST, 2);

	char LeaderboardNameANSI[EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH];
	FCStringAnsi::Strncpy(LeaderboardNameANSI, TCHAR_TO_UTF8(*BoardName), EOS_LEADERBOARD_STAT_NAME_MAX_LENGTH);
	Options.LeaderboardId = LeaderboardNameANSI;

	EOS_Async(EOS_Leaderboards_QueryLeaderboardRanks, LeaderboardsHandle, Options, MoveTemp(Promise));
}

}

void FLeaderboardsEOSGS::ReadEntriesInRange(uint32 StartIndex, uint32 EndIndex, TArray<FLeaderboardEntry>& OutEntries)
{
	for (uint32 i = StartIndex; i <= EndIndex; ++i)
	{
		EOS_Leaderboards_CopyLeaderboardRecordByIndexOptions CopyRecordOptions;
		CopyRecordOptions.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDRECORDBYINDEX_API_LATEST, 2);
		CopyRecordOptions.LeaderboardRecordIndex = i;

		EOS_Leaderboards_LeaderboardRecord* LeaderboardRecord = nullptr;
		EOS_EResult CopyResult = EOS_Leaderboards_CopyLeaderboardRecordByIndex(LeaderboardsHandle, &CopyRecordOptions, &LeaderboardRecord);
		if (CopyResult == EOS_EResult::EOS_Success)
		{
			FLeaderboardEntry& LeaderboardEntry = OutEntries.Emplace_GetRef();
			LeaderboardEntry.AccountId = FindAccountIdChecked(Services.GetServicesProvider(), LeaderboardRecord->UserId);
			LeaderboardEntry.Rank = LeaderboardRecord->Rank;
			LeaderboardEntry.Score = LeaderboardRecord->Score;

			EOS_Leaderboards_LeaderboardRecord_Release(LeaderboardRecord);
		}
	}
}

TOnlineAsyncOpHandle<FReadEntriesAroundRank> FLeaderboardsEOSGS::ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundRank> Op = GetJoinableOp<FReadEntriesAroundRank>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FReadEntriesAroundRank>& InAsyncOp)
		{
			const FReadEntriesAroundRank::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
			else if (Params.Rank >= EOS_LEADERBOARD_MAX_ENTRIES || Params.Limit == 0)
			{
				InAsyncOp.SetError(Errors::InvalidParams());
			}
		})
		.Then([this](TOnlineAsyncOp<FReadEntriesAroundRank>& InAsyncOp, TPromise<const EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo*>&& Promise)
		{
			const FReadEntriesAroundRank::Params& Params = InAsyncOp.GetParams();
			Private::QueryLeaderboardsEOS(LeaderboardsHandle, Params.LocalAccountId, Params.BoardName, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FReadEntriesAroundRank>& InAsyncOp, const EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Leaderboards_QueryLeaderboardRanks failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			const FReadEntriesAroundRank::Params& Params = InAsyncOp.GetParams();

			FReadEntriesAroundRank::Result Result;
			int32 StartIndex = Params.Rank;
			int32 EndIndex = StartIndex + Params.Limit;
			EndIndex = FMath::Clamp(EndIndex, 0, EOS_LEADERBOARD_MAX_ENTRIES-1);
			ReadEntriesInRange(StartIndex, EndIndex, Result.Entries);
			InAsyncOp.SetResult(MoveTemp(Result));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesAroundUser> FLeaderboardsEOSGS::ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundUser> Op = GetJoinableOp<FReadEntriesAroundUser>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FReadEntriesAroundUser>& InAsyncOp)
		{
			const FReadEntriesAroundUser::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
			else if (Params.Limit > EOS_LEADERBOARD_MAX_ENTRIES || Params.Limit == 0 || Params.Offset >= EOS_LEADERBOARD_MAX_ENTRIES || Params.Offset <= -EOS_LEADERBOARD_MAX_ENTRIES)
			{
				InAsyncOp.SetError(Errors::InvalidParams());
			}
		})
		.Then([this](TOnlineAsyncOp<FReadEntriesAroundUser>& InAsyncOp, TPromise<const EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo*>&& Promise)
		{
			const FReadEntriesAroundUser::Params& Params = InAsyncOp.GetParams();
			Private::QueryLeaderboardsEOS(LeaderboardsHandle, Params.LocalAccountId, Params.BoardName, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FReadEntriesAroundUser>& InAsyncOp, const EOS_Leaderboards_OnQueryLeaderboardRanksCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Leaderboards_QueryLeaderboardRanks failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			const FReadEntriesAroundUser::Params& Params = InAsyncOp.GetParams();

			FReadEntriesAroundUser::Result Result;

			EOS_Leaderboards_LeaderboardRecord* LeaderboardRecord = nullptr;
			EOS_Leaderboards_CopyLeaderboardRecordByUserIdOptions CopyRecordByUserIdOptions;
			CopyRecordByUserIdOptions.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_LEADERBOARDS_COPYLEADERBOARDRECORDBYUSERID_API_LATEST, 2);
			CopyRecordByUserIdOptions.UserId = GetProductUserIdChecked(Params.AccountId);
			EOS_EResult CopyResult = EOS_Leaderboards_CopyLeaderboardRecordByUserId(LeaderboardsHandle, &CopyRecordByUserIdOptions, &LeaderboardRecord);
			if (CopyResult == EOS_EResult::EOS_Success)
			{
				int32 StartIndex = (int32)LeaderboardRecord->Rank + Params.Offset;
				StartIndex = FMath::Clamp(StartIndex, 0, EOS_LEADERBOARD_MAX_ENTRIES - 1);
				int32 EndIndex = StartIndex + Params.Limit;
				EndIndex = FMath::Clamp(EndIndex, 0, EOS_LEADERBOARD_MAX_ENTRIES - 1);
				ReadEntriesInRange(StartIndex, EndIndex, Result.Entries);

				EOS_Leaderboards_LeaderboardRecord_Release(LeaderboardRecord);
			}

			InAsyncOp.SetResult(MoveTemp(Result));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

/* UE::Online */ }
