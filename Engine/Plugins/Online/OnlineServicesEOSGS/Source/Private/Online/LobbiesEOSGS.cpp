// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LobbiesEOSGS.h"

#include "EOSShared.h"
#include "Online/AuthEOSGS.h"
#include "Online/LobbiesEOSGSTypes.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "IEOSSDKManager.h"

#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "eos_lobby.h"

#define UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA TEXT("LobbyData")
#define UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS TEXT("LobbyDetails")
#define UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_ID_STRING TEXT("LobbyIdString")
#define UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_SEARCH TEXT("LobbySearch")

namespace UE::Online {

static const int32 MaxAttributeSize = 1000;

FLobbiesEOSGS::FLobbiesEOSGS(FOnlineServicesEpicCommon& InServices)
	: FLobbiesCommon(InServices)
{
}

void FLobbiesEOSGS::Initialize()
{
	FLobbiesCommon::Initialize();

	IEOSSDKManager* SDKManager = IEOSSDKManager::Get();
	check(SDKManager);

	EOS_HLobby LobbyInterfaceHandle = EOS_Platform_GetLobbyInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(LobbyInterfaceHandle != nullptr);

	LobbyPrerequisites = MakeShared<FLobbyPrerequisitesEOS>(
		LobbyInterfaceHandle,
		Services.Get<IEpicProductUserIdResolver>(),
		FLobbyBucketIdEOS(SDKManager->GetProductName(), GetBuildUniqueId()),
		SchemaRegistry);

	LobbyDataRegistry = MakeShared<FLobbyDataRegistryEOS>(LobbyPrerequisites.ToSharedRef());

	RegisterHandlers();
}

void FLobbiesEOSGS::PreShutdown()
{
	Super::PreShutdown();

	UnregisterHandlers();

	LobbyDataRegistry = nullptr;
	LobbyPrerequisites = nullptr;
}

TOnlineAsyncOpHandle<FCreateLobby> FLobbiesEOSGS::CreateLobby(FCreateLobby::Params&& InParams)
{
	// Helper lambda for simplifying error handling.
	auto DestroyLobbyDuringCreate = [this](
		TOnlineAsyncOp<FCreateLobby>& InAsyncOp,
		FAccountId LocalAccountId,
		const FString& LobbyIdString,
		FOnlineError ErrorResult) -> TFuture<void>
	{
		FLobbiesDestroyLobbyImpl::Params DestroyLobbyParams;
		DestroyLobbyParams.LobbyIdString = LobbyIdString;
		DestroyLobbyParams.LocalAccountId = LocalAccountId;

		TPromise<void> Promise;
		auto Future = Promise.GetFuture();

		DestroyLobbyImpl(MoveTemp(DestroyLobbyParams))
		.Next(
		[
			InAsyncOp = InAsyncOp.AsShared(),
			LocalAccountId,
			LobbyIdString,
			ErrorResult = MoveTemp(ErrorResult),
			Promise = MoveTemp(Promise)
		](TDefaultErrorResult<FLobbiesDestroyLobbyImpl>&& Result) mutable
		{
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] DestroyLobbyImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
					*ToLogString(LocalAccountId), *LobbyIdString, *Result.GetErrorValue().GetLogString());
			}

			// Set operation error to the passed in error value.
			InAsyncOp->SetError(MoveTemp(ErrorResult));
			Promise.EmplaceValue();
		});

		return Future;
	};

	FSchemaVariant Attribute;
	InParams.Attributes.RemoveAndCopyValue(LOBBIESEOSGS_LOBBY_ID_OVERRIDE_ATTRIBUTE_KEY, Attribute);
	FString LobbyIdOverrideStr = Attribute.VariantType == ESchemaAttributeType::String ? Attribute.GetString() : FString();

	TOnlineAsyncOpRef<FCreateLobby> Op = GetOp<FCreateLobby>(MoveTemp(InParams));
	const FCreateLobby::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	// Check that requested lobby schema exists.
	if (!SchemaRegistry->GetDefinition(Params.SchemaId).IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] Failed: Unknown lobby schema [%ls]", *Params.SchemaId.ToString().ToLower());
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	// Check for an existing presence lobby
	if (Params.bPresenceEnabled)
	{
		TOnlineResult<FGetPresenceLobby> GetPresenceLobbyResult = GetPresenceLobby({ Params.LocalAccountId });
		if (GetPresenceLobbyResult.IsOk())
		{
			UE_LOGF(LogTemp, Warning, "[%ls] Failed: bPresenceEnabled was set to true, but there is already a presence Lobby", UTF8_TO_TCHAR(__FUNCTION__));
			Op->SetError(Errors::InvalidState());
			return Op->GetHandle();
		}
	}

	// Start operation.
	// Step 1: Call create lobby.
	Op->Then([this, LobbyIdOverrideStr](TOnlineAsyncOp<FCreateLobby>& InAsyncOp, TPromise<const EOS_Lobby_CreateLobbyCallbackInfo*>&& Promise)
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();
		const FLobbyBucketIdTranslator<ELobbyTranslationType::ToService> BucketTanslator(LobbyPrerequisites->BucketId);

		// The lobby will be created as invitation only.
		// Once all local members are joined and the lobby attributes have been set the privacy setting will be moved to the user setting.

		EOS_Lobby_CreateLobbyOptions CreateLobbyOptions = {};
		CreateLobbyOptions.ApiVersion = 10;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_CREATELOBBY_API_LATEST, 10);
		CreateLobbyOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
		CreateLobbyOptions.MaxLobbyMembers = Params.MaxMembers;
		// Prevent lobby from appearing in search results until fully created with all user attributes.
		CreateLobbyOptions.PermissionLevel = EOS_ELobbyPermissionLevel::EOS_LPL_INVITEONLY;
		CreateLobbyOptions.bPresenceEnabled = Params.bPresenceEnabled;
		CreateLobbyOptions.bAllowInvites = true; // todo: handle
		CreateLobbyOptions.BucketId = BucketTanslator.GetBucketIdEOS();
		CreateLobbyOptions.bDisableHostMigration = false; // todo: handle
		CreateLobbyOptions.bEnableRTCRoom = false; // todo: handle
		CreateLobbyOptions.bEnableJoinById = false; // todo: handle
		CreateLobbyOptions.bRejoinAfterKickRequiresInvite = false; // todo: handle
		CreateLobbyOptions.bCrossplayOptOut = false;
		CreateLobbyOptions.RTCRoomJoinActionType = EOS_ELobbyRTCRoomJoinActionType::EOS_LRRJAT_AutomaticJoin;

		const auto LobbyIdOverrideConverter = StringCast<UTF8CHAR>(*LobbyIdOverrideStr);
		CreateLobbyOptions.LobbyId = LobbyIdOverrideConverter.Length() ? (const char*)LobbyIdOverrideConverter.Get() : nullptr;
		
		EOS_Async(EOS_Lobby_CreateLobby, LobbyPrerequisites->LobbyInterfaceHandle, CreateLobbyOptions, MoveTemp(Promise)); 
	})
	// Step 2: Handle errors and attach the lobby id to the operation data.
	.Then([this](TOnlineAsyncOp<FCreateLobby>& InAsyncOp, const EOS_Lobby_CreateLobbyCallbackInfo* Data)
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();

		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] EOS_Lobby_CreateLobby Failed: User[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LexToString(Data->ResultCode));
			InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
		}
		else
		{
			// Store the lobby id string on the operation data. This is used to destroy the lobby when creating the lobby structures fails.
			InAsyncOp.Data.Set<FString>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_ID_STRING, FString(UTF8_TO_TCHAR(Data->LobbyId)));
		}
	})
	// Step 3: Create the lobby details object from the lobby id.
	.Then([this, DestroyLobbyDuringCreate](TOnlineAsyncOp<FCreateLobby>& InAsyncOp)
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();
		const FString& LobbyIdString = GetOpDataChecked<FString>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_ID_STRING);

		// Try to create the lobby details object.
		TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> Result =
			FLobbyDetailsEOS::CreateFromLobbyId(LobbyPrerequisites.ToSharedRef(), Params.LocalAccountId, LobbyIdString);
		if (Result.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] FLobbyDetailsEOS::CreateFromLobbyId Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyIdString, *Result.GetErrorValue().GetLogString());

			return DestroyLobbyDuringCreate(InAsyncOp, Params.LocalAccountId, LobbyIdString, MoveTemp(Result.GetErrorValue()));
		}
		else
		{
			// Attach the lobby details to the operation.
			InAsyncOp.Data.Set<TSharedRef<FLobbyDetailsEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS, Result.GetOkValue());
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	// Step 4: Create the lobby data object from the lobby details.
	.Then([this](TOnlineAsyncOp<FCreateLobby>& InAsyncOp)
	{
		TSharedRef<FLobbyDetailsEOS> LobbyDetails = GetOpDataChecked<TSharedRef<FLobbyDetailsEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS);
		return LobbyDataRegistry->FindOrCreateFromLobbyDetails(InAsyncOp.GetParams().LocalAccountId, LobbyDetails);
	})
	// Step 5: Handle errors and store the lobby data on the async op properties.
	.Then([this, DestroyLobbyDuringCreate](TOnlineAsyncOp<FCreateLobby>& InAsyncOp, TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>&& Result)
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();
		const FString& LobbyIdString = GetOpDataChecked<FString>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_ID_STRING);

		if (Result.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyIdString, *Result.GetErrorValue().GetLogString());

			return DestroyLobbyDuringCreate(InAsyncOp, Params.LocalAccountId, LobbyIdString, MoveTemp(Result.GetErrorValue()));
		}
		else
		{
			// Store lobby data on the operation.
			InAsyncOp.Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, MoveTemp(Result.GetOkValue()));
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	// Step 6: Set lobby and creator attributes, change to user lobby privacy setting.
	.Then([this](TOnlineAsyncOp<FCreateLobby>& InAsyncOp)
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.Attributes = { Params.Attributes, {} };
		PrepareParams.ClientChanges.MemberAttributes = { Params.UserAttributes, {} };
		PrepareParams.ClientChanges.JoinPolicy = Params.JoinPolicy;
		PrepareParams.ClientChanges.LobbySchema = Params.SchemaId;
		PrepareParams.ClientChanges.LocalName = Params.LocalName;
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(FLobbiesModifyLobbyDataImpl::Result{}).GetFuture();
		}

		FLobbiesModifyLobbyDataImpl::Params ModifyLobbyDataParams;
		ModifyLobbyDataParams.LobbyData = LobbyData;
		ModifyLobbyDataParams.LocalAccountId = Params.LocalAccountId;
		ModifyLobbyDataParams.ServiceChanges = MoveTemp(PrepareResult.GetOkValue().ServiceChanges);
		return ModifyLobbyDataImpl(MoveTemp(ModifyLobbyDataParams));
	})
	// Step 7: Handle result
	.Then([this, DestroyLobbyDuringCreate](TOnlineAsyncOp<FCreateLobby>& InAsyncOp, TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>&& Result) mutable
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		if (Result.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::CreateLobby] ModifyLobbyDataImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());

			return DestroyLobbyDuringCreate(InAsyncOp, Params.LocalAccountId, LobbyData->GetLobbyIdString(), MoveTemp(Result.GetErrorValue()));
		}
		else
		{
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	// Step 8: Add the lobby to the active list, apply changes to the cached lobby object, and signal notifications.
	.Then([this](TOnlineAsyncOp<FCreateLobby>& InAsyncOp)
	{
		const FCreateLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Mark the lobby active.
		AddActiveLobby(Params.LocalAccountId, Params.bPresenceEnabled, LobbyData);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Log, "[FLobbiesEOSGS::CreateLobby] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FCreateLobby::Result{LobbyData->GetLobbyClientData()->GetPublicDataPtr()});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FFindLobbies> FLobbiesEOSGS::FindLobbies(FFindLobbies::Params&& InParams)
{
	TOnlineAsyncOpRef<FFindLobbies> Op = GetOp<FFindLobbies>(MoveTemp(InParams));
	const FFindLobbies::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::FindLobbies] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	// Invalidate previous search results.
	ActiveSearchResults.Remove(Params.LocalAccountId);

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FFindLobbies>& InAsyncOp)
	{
		return FLobbySearchEOS::Create(LobbyPrerequisites.ToSharedRef(), LobbyDataRegistry.ToSharedRef(), InAsyncOp.GetParams());
	})
	.Then([this](TOnlineAsyncOp<FFindLobbies>& InAsyncOp, TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>> Result)
	{
		if (Result.IsError())
		{
			const FFindLobbies::Params& Params = InAsyncOp.GetParams();

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] FLobbySearchEOS::Create Failed: User[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
		}
		else
		{
			InAsyncOp.Data.Set<TSharedRef<FLobbySearchEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_SEARCH, Result.GetOkValue());
		}
	})
	.Then([this](TOnlineAsyncOp<FFindLobbies>& InAsyncOp)
	{
		const FFindLobbies::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbySearchEOS>& LobbySearch = GetOpDataChecked<TSharedRef<FLobbySearchEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_SEARCH);

		ActiveSearchResults.Add(InAsyncOp.GetParams().LocalAccountId, LobbySearch);

		UE_LOGF(LogOnlineServices, Log, "[FLobbiesEOSGS::FindLobbies] Succeeded: User[%ls]",
			*ToLogString(Params.LocalAccountId));
		InAsyncOp.SetResult(FFindLobbies::Result{ LobbySearch->GetLobbyResults() });
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FJoinLobby> FLobbiesEOSGS::JoinLobby(FJoinLobby::Params&& InParams)
{
	// Helper lambda for simplifying error handling.
	auto LeaveLobbyDuringJoin = [this](
		TOnlineAsyncOp<FJoinLobby>& InAsyncOp,
		const FAccountId LocalAccountId,
		const TSharedPtr<FLobbyDataEOS> LobbyData,
		FOnlineError ErrorResult) -> TFuture<void>
	{
		// Failed to set attributes - leave the lobby.
		FLobbiesLeaveLobbyImpl::Params LeaveLobbyParams;
		LeaveLobbyParams.LobbyData = LobbyData;
		LeaveLobbyParams.LocalAccountId = LocalAccountId;

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		LeaveLobbyImpl(MoveTemp(LeaveLobbyParams))
		.Next(
		[
			LocalAccountId = LocalAccountId,
			LobbyData,
			InAsyncOp = InAsyncOp.AsShared(),
			ErrorResult = MoveTemp(ErrorResult),
			Promise = MoveTemp(Promise)
		](TDefaultErrorResult<FLobbiesLeaveLobbyImpl>&& Result) mutable
		{
			if (Result.IsError())
			{
				UE_LOGF(LogTemp, Warning, "[FLobbiesEOSGS::JoinLobby] LeaveLobbyImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
					*ToLogString(LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			}

			InAsyncOp->SetError(MoveTemp(ErrorResult));
			Promise.EmplaceValue();
		});

		return Future;
	};

	TOnlineAsyncOpRef<FJoinLobby> Op = GetOp<FJoinLobby>(MoveTemp(InParams));
	const FJoinLobby::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	// Check for an existing presence lobby
	if (Params.bPresenceEnabled)
	{
		TOnlineResult<FGetPresenceLobby> GetPresenceLobbyResult = GetPresenceLobby({ Params.LocalAccountId });
		if (GetPresenceLobbyResult.IsOk())
		{
			UE_LOGF(LogTemp, Warning, "[%ls] Failed: bPresenceEnabled was set to true, but there is already a presence Lobby", UTF8_TO_TCHAR(__FUNCTION__));
			Op->SetError(Errors::InvalidState());
			return Op->GetHandle();
		}
	}

	// Lobby data must already exist through local creation, invitation, search, or UI event when joining.
	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Setup lobby details - Prefer UI event before invitation before search result.
	TSharedPtr<FLobbyDetailsEOS> LobbyDetails = LobbyData->GetUserLobbyDetails(Params.LocalAccountId);
	if (!LobbyDetails)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] Failed: Unable to find lobby details for user. User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDetailsEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS, LobbyDetails.ToSharedRef());

	// Check that lobby is compatible with the running game.
	if (!LobbyDetails->IsBucketCompatible())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] Failed: Lobby is not compatible with the client version. Lobby[%ls], Bucket[%ls], ExpectedBucket[%ls].",
			*LobbyData->GetLobbyIdString(), *ToLogString(LobbyDetails->GetInfo()->GetBucketId()), *ToLogString(LobbyPrerequisites->BucketId));
		Op->SetError(Errors::IncompatibleVersion());
		return Op->GetHandle();
	}

	// Start operation.
	// Step 1. Join the lobby.
	Op->Then([this](TOnlineAsyncOp<FJoinLobby>& InAsyncOp, TPromise<const EOS_Lobby_JoinLobbyCallbackInfo*>&& Promise)
	{
		const FJoinLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDetailsEOS>& LobbyDetails = GetOpDataChecked<TSharedRef<FLobbyDetailsEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS);

		EOS_Lobby_JoinLobbyOptions JoinLobbyOptions = {};
		JoinLobbyOptions.ApiVersion = 5;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_JOINLOBBY_API_LATEST, 5);
		JoinLobbyOptions.LobbyDetailsHandle = LobbyDetails->GetEOSHandle();
		JoinLobbyOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
		JoinLobbyOptions.bPresenceEnabled = Params.bPresenceEnabled;
		JoinLobbyOptions.LocalRTCOptions = nullptr;
		JoinLobbyOptions.bCrossplayOptOut = false;
		JoinLobbyOptions.RTCRoomJoinActionType = EOS_ELobbyRTCRoomJoinActionType::EOS_LRRJAT_AutomaticJoin;

		EOS_Async(EOS_Lobby_JoinLobby, LobbyPrerequisites->LobbyInterfaceHandle, JoinLobbyOptions, MoveTemp(Promise));
	})
	// Step 2. Handle join result.
	.Then([this](TOnlineAsyncOp<FJoinLobby>& InAsyncOp, const EOS_Lobby_JoinLobbyCallbackInfo* Data)
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			const FJoinLobby::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] EOS_Lobby_JoinLobby Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *LexToString(Data->ResultCode));
			InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
			return;
		}
	})	
	// Step 3. Promote lobby details source to active.
	.Then([this, LeaveLobbyDuringJoin](TOnlineAsyncOp<FJoinLobby>& InAsyncOp)
	{
		const FJoinLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Create an active lobby details object associated with the lobby data that we can use when we get lobby update notifications.
		const FString& LobbyIdString = LobbyData->GetLobbyIdString();
		InAsyncOp.Data.Set<FString>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_ID_STRING, LobbyIdString);

		// Try to create the lobby details object.
		TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> Result =
			FLobbyDetailsEOS::CreateFromLobbyId(LobbyPrerequisites.ToSharedRef(), Params.LocalAccountId, LobbyIdString);
		if (Result.IsError())
		{
			UE_LOGF(LogTemp, Warning, "[FLobbiesEOSGS::JoinLobby] FLobbyDetailsEOS::CreateFromLobbyId Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyIdString, *Result.GetErrorValue().GetLogString());
			return LeaveLobbyDuringJoin(InAsyncOp, Params.LocalAccountId, LobbyData, MoveTemp(Result.GetErrorValue()));
		}
		else
		{
			// Attach the lobby details to the operation so we can register them in the next step.
			InAsyncOp.Data.Set<TSharedRef<FLobbyDetailsEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS, Result.GetOkValue());
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})	
	// Step 4: Create the lobby data object from the lobby details.
	.Then([this](TOnlineAsyncOp<FJoinLobby>& InAsyncOp)
	{
		TSharedRef<FLobbyDetailsEOS> LobbyDetails = GetOpDataChecked<TSharedRef<FLobbyDetailsEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS);
		return LobbyDataRegistry->FindOrCreateFromLobbyDetails(InAsyncOp.GetParams().LocalAccountId, LobbyDetails);
	})
	// Step 5: Handle possible errors from FindOrCreateFromLobbyDetails.
	.Then([this, LeaveLobbyDuringJoin](TOnlineAsyncOp<FJoinLobby>& InAsyncOp, TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>&& Result)
	{
		const FJoinLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		if (Result.IsError())
		{
			UE_LOGF(LogTemp, Warning, "[FLobbiesEOSGS::JoinLobby] FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			return LeaveLobbyDuringJoin(InAsyncOp, Params.LocalAccountId, LobbyData, MoveTemp(Result.GetErrorValue()));
		}
		else
		{		
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	// Step 6. Set member attributes.
	.Then([this](TOnlineAsyncOp<FJoinLobby>& InAsyncOp)
	{
		const FJoinLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.LocalName = Params.LocalName;
		PrepareParams.ClientChanges.MemberAttributes = { Params.UserAttributes, {} };
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(FLobbiesModifyLobbyDataImpl::Result{}).GetFuture();
		}

		FLobbiesModifyLobbyDataImpl::Params ModifyLobbyDataParams;
		ModifyLobbyDataParams.LobbyData = LobbyData;
		ModifyLobbyDataParams.LocalAccountId = Params.LocalAccountId;
		ModifyLobbyDataParams.ServiceChanges = MoveTemp(PrepareResult.GetOkValue().ServiceChanges);
		return ModifyLobbyDataImpl(MoveTemp(ModifyLobbyDataParams));
	})
	// Step 7. Handle result.
	.Then([this, LeaveLobbyDuringJoin](TOnlineAsyncOp<FJoinLobby>& InAsyncOp, TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FJoinLobby::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::JoinLobby] ModifyLobbyMemberDataImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());

			return LeaveLobbyDuringJoin(InAsyncOp, Params.LocalAccountId, LobbyData, MoveTemp(Result.GetErrorValue()));
		}
		else
		{
			return MakeFulfilledPromise<void>().GetFuture();
		}
	})
	// Step 8. Bookkeeping and notifications.
	.Then([this](TOnlineAsyncOp<FJoinLobby>& InAsyncOp)
	{
		const FJoinLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Remove the invitation if this is a join in response to an invite.
		if (TSharedPtr<FLobbyInviteDataEOS> InviteData = GetActiveInvite(Params.LocalAccountId, Params.LobbyId))
		{
			RemoveActiveInvite(InviteData.ToSharedRef());
		}

		// Mark the lobby active.
		AddActiveLobby(Params.LocalAccountId, Params.bPresenceEnabled, LobbyData);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Log, "[FLobbiesEOSGS::JoinLobby] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FJoinLobby::Result{ LobbyData->GetLobbyClientData()->GetPublicDataPtr()});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FLeaveLobby> FLobbiesEOSGS::LeaveLobby(FLeaveLobby::Params&& InParams)
{
	TOnlineAsyncOpRef<FLeaveLobby> Op = GetOp<FLeaveLobby>(MoveTemp(InParams));
	const FLeaveLobby::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::LeaveLobby] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::LeaveLobby] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::LeaveLobby] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FLeaveLobby>& InAsyncOp)
	{
		const FLeaveLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.LocalUserLeaveReason = ELobbyMemberLeaveReason::Left;
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMember] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>>(FLobbiesLeaveLobbyImpl::Result{}).GetFuture();
		}

		FLobbiesLeaveLobbyImpl::Params LeaveParams;
		LeaveParams.LobbyData = LobbyData;
		LeaveParams.LocalAccountId = Params.LocalAccountId;
		return LeaveLobbyImpl(MoveTemp(LeaveParams));
	})
	.Then([this](TOnlineAsyncOp<FLeaveLobby>& InAsyncOp, TDefaultErrorResult<FLobbiesLeaveLobbyImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FLeaveLobby::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::LeaveLobby] LeaveLobbyImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());

			// When leaving a lobby consume the error and report success.
			//InAsyncOp.SetError(Result.GetErrorValue());

			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FLeaveLobby>& InAsyncOp)
	{
		const FLeaveLobby::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Remove the lobby from the active list for the user.
		// The lobby data will be cleaned up once all references are removed.
		RemoveActiveLobby(Params.LocalAccountId, LobbyData);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Log, "[FLobbiesEOSGS::LeaveLobby] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FLeaveLobby::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FInviteLobbyMember> FLobbiesEOSGS::InviteLobbyMember(FInviteLobbyMember::Params&& InParams)
{
	TOnlineAsyncOpRef<FInviteLobbyMember> Op = GetOp<FInviteLobbyMember>(MoveTemp(InParams));
	const FInviteLobbyMember::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMember] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMember] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMember] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FInviteLobbyMember>& InAsyncOp)
	{
		const FInviteLobbyMember::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbiesInviteLobbyMemberImpl::Params InviteParams;
		InviteParams.LobbyData = LobbyData;
		InviteParams.LocalAccountId = Params.LocalAccountId;
		InviteParams.TargetAccountId = Params.TargetAccountId;
		return InviteLobbyMemberImpl(MoveTemp(InviteParams));
	})
	.Then([this](TOnlineAsyncOp<FInviteLobbyMember>& InAsyncOp, TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FInviteLobbyMember::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMember] InviteLobbyMemberImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FInviteLobbyMember>& InAsyncOp)
	{
		const FInviteLobbyMember::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::InviteLobbyMember] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FInviteLobbyMember::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FDeclineLobbyInvitation> FLobbiesEOSGS::DeclineLobbyInvitation(FDeclineLobbyInvitation::Params&& InParams)
{
	TOnlineAsyncOpRef<FDeclineLobbyInvitation> Op = GetOp<FDeclineLobbyInvitation>(MoveTemp(InParams));
	const FDeclineLobbyInvitation::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitation] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitation] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogTemp, Warning, "[FLobbiesEOSGS::JoinLobby] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FDeclineLobbyInvitation>& InAsyncOp)
	{
		const FDeclineLobbyInvitation::Params& Params = InAsyncOp.GetParams();

		FLobbiesDeclineLobbyInvitationImpl::Params DeclineParams;
		DeclineParams.LocalAccountId = Params.LocalAccountId;
		DeclineParams.LobbyId = Params.LobbyId;
		return DeclineLobbyInvitationImpl(MoveTemp(DeclineParams));
	})
	.Then([this](TOnlineAsyncOp<FDeclineLobbyInvitation>& InAsyncOp, TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FDeclineLobbyInvitation::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitation] DeclineLobbyInvitationImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FDeclineLobbyInvitation>& InAsyncOp)
	{
		const FDeclineLobbyInvitation::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::DeclineLobbyInvitation] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FDeclineLobbyInvitation::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FKickLobbyMember> FLobbiesEOSGS::KickLobbyMember(FKickLobbyMember::Params&& InParams)
{
	TOnlineAsyncOpRef<FKickLobbyMember> Op = GetOp<FKickLobbyMember>(MoveTemp(InParams));
	const FKickLobbyMember::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMember] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMember] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMember] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FKickLobbyMember>& InAsyncOp)
	{
		const FKickLobbyMember::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.KickedTargetMember = Params.TargetAccountId;
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMember] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>>(FLobbiesKickLobbyMemberImpl::Result{}).GetFuture();
		}

		FLobbiesKickLobbyMemberImpl::Params KickParams;
		KickParams.LobbyData = LobbyData;
		KickParams.LocalAccountId = Params.LocalAccountId;
		KickParams.TargetAccountId = *PrepareResult.GetOkValue().ServiceChanges.KickedTargetMember;
		return KickLobbyMemberImpl(MoveTemp(KickParams));
	})
	.Then([this](TOnlineAsyncOp<FKickLobbyMember>& InAsyncOp, TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FKickLobbyMember::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMember] KickLobbyMemberImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
		}
	})
	.Then([this](TOnlineAsyncOp<FKickLobbyMember>& InAsyncOp)
	{
		const FKickLobbyMember::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::KickLobbyMember] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FKickLobbyMember::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FPromoteLobbyMember> FLobbiesEOSGS::PromoteLobbyMember(FPromoteLobbyMember::Params&& InParams)
{
	// Check prerequisites.
	TOnlineAsyncOpRef<FPromoteLobbyMember> Op = GetOp<FPromoteLobbyMember>(MoveTemp(InParams));
	const FPromoteLobbyMember::Params& Params = Op->GetParams();

	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::FPromoteLobbyMember] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::FPromoteLobbyMember] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::FPromoteLobbyMember] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FPromoteLobbyMember>& InAsyncOp)
	{
		const FPromoteLobbyMember::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.OwnerAccountId = Params.TargetAccountId;
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::FPromoteLobbyMember] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>>(FLobbiesPromoteLobbyMemberImpl::Result{}).GetFuture();
		}

		FLobbiesPromoteLobbyMemberImpl::Params PromoteParams;
		PromoteParams.LobbyData = LobbyData;
		PromoteParams.LocalAccountId = Params.LocalAccountId;
		PromoteParams.TargetAccountId = *PrepareResult.GetOkValue().ServiceChanges.OwnerAccountId;
		return PromoteLobbyMemberImpl(MoveTemp(PromoteParams));
	})
	.Then([this](TOnlineAsyncOp<FPromoteLobbyMember>& InAsyncOp, TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FPromoteLobbyMember::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::FPromoteLobbyMember] PromoteLobbyMemberImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FPromoteLobbyMember>& InAsyncOp)
	{
		const FPromoteLobbyMember::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::FPromoteLobbyMember] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FPromoteLobbyMember::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbyJoinPolicy> FLobbiesEOSGS::ModifyLobbyJoinPolicy(FModifyLobbyJoinPolicy::Params&& InParams)
{
	TOnlineAsyncOpRef<FModifyLobbyJoinPolicy> Op = GetOp<FModifyLobbyJoinPolicy>(MoveTemp(InParams));
	const FModifyLobbyJoinPolicy::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyJoinPolicy] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyJoinPolicy] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyJoinPolicy] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FModifyLobbyJoinPolicy>& InAsyncOp)
	{
		const FModifyLobbyJoinPolicy::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.JoinPolicy = Params.JoinPolicy;
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyJoinPolicy] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(FLobbiesModifyLobbyDataImpl::Result{}).GetFuture();
		}

		FLobbiesModifyLobbyDataImpl::Params ModifyLobbyDataParams;
		ModifyLobbyDataParams.LobbyData = LobbyData;
		ModifyLobbyDataParams.LocalAccountId = Params.LocalAccountId;
		ModifyLobbyDataParams.ServiceChanges = MoveTemp(PrepareResult.GetOkValue().ServiceChanges);
		return ModifyLobbyDataImpl(MoveTemp(ModifyLobbyDataParams));
	})
	.Then([this](TOnlineAsyncOp<FModifyLobbyJoinPolicy>& InAsyncOp, TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FModifyLobbyJoinPolicy::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyJoinPolicy] ModifyLobbyDataImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FModifyLobbyJoinPolicy>& InAsyncOp)
	{
		const FModifyLobbyJoinPolicy::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::ModifyLobbyJoinPolicy] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FModifyLobbyJoinPolicy::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbyAttributes> FLobbiesEOSGS::ModifyLobbyAttributes(FModifyLobbyAttributes::Params&& InParams)
{
	TOnlineAsyncOpRef<FModifyLobbyAttributes> Op = GetOp<FModifyLobbyAttributes>(MoveTemp(InParams));
	const FModifyLobbyAttributes::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyAttributes] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyAttributes] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyAttributes] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FModifyLobbyAttributes>& InAsyncOp)
	{
		const FModifyLobbyAttributes::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.Attributes = { Params.UpdatedAttributes, Params.RemovedAttributes };
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyAttributes] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(FLobbiesModifyLobbyDataImpl::Result{}).GetFuture();
		}

		FLobbiesModifyLobbyDataImpl::Params ModifyLobbyDataParams;
		ModifyLobbyDataParams.LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
		ModifyLobbyDataParams.LocalAccountId = Params.LocalAccountId;
		ModifyLobbyDataParams.ServiceChanges = MoveTemp(PrepareResult.GetOkValue().ServiceChanges);
		return ModifyLobbyDataImpl(MoveTemp(ModifyLobbyDataParams));
	})
	.Then([this](TOnlineAsyncOp<FModifyLobbyAttributes>& InAsyncOp, TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FModifyLobbyAttributes::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyAttributes] ModifyLobbyDataImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FModifyLobbyAttributes>& InAsyncOp)
	{
		const FModifyLobbyAttributes::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::ModifyLobbyAttributes] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FModifyLobbyAttributes::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbyMemberAttributes> FLobbiesEOSGS::ModifyLobbyMemberAttributes(FModifyLobbyMemberAttributes::Params&& InParams)
{
	TOnlineAsyncOpRef<FModifyLobbyMemberAttributes> Op = GetOp<FModifyLobbyMemberAttributes>(MoveTemp(InParams));
	const FModifyLobbyMemberAttributes::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyMemberAttributes] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyMemberAttributes] Failed: Lobby id is invalid.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Params.LobbyId);
	if (!LobbyData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyMemberAttributes] Failed: Unable to find lobby data. LobbyId[%ls]", *ToLogString(Params.LobbyId));
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}
	Op->Data.Set<TSharedRef<FLobbyDataEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA, LobbyData.ToSharedRef());

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FModifyLobbyMemberAttributes>& InAsyncOp)
	{
		const FModifyLobbyMemberAttributes::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		FLobbyClientDataPrepareClientChanges::Params PrepareParams;
		PrepareParams.LocalAccountId = Params.LocalAccountId;
		PrepareParams.ClientChanges.MemberAttributes = { Params.UpdatedAttributes, Params.RemovedAttributes };
		TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = LobbyData->GetLobbyClientData()->PrepareClientChanges(MoveTemp(PrepareParams));
		if (PrepareResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyMemberAttributes] PrepareClientChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *PrepareResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareResult.GetErrorValue()));
			return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(FLobbiesModifyLobbyDataImpl::Result{}).GetFuture();
		}

		FLobbiesModifyLobbyDataImpl::Params ModifyLobbyDataParams;
		ModifyLobbyDataParams.LobbyData = LobbyData;
		ModifyLobbyDataParams.LocalAccountId = Params.LocalAccountId;
		ModifyLobbyDataParams.ServiceChanges = MoveTemp(PrepareResult.GetOkValue().ServiceChanges);
		return ModifyLobbyDataImpl(MoveTemp(ModifyLobbyDataParams));
	})
	.Then([this](TOnlineAsyncOp<FModifyLobbyMemberAttributes>& InAsyncOp, TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>&& Result)
	{
		if (Result.IsError())
		{
			const FModifyLobbyMemberAttributes::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyMemberAttributes] ModifyLobbyMemberDataImpl Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(Result.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FModifyLobbyMemberAttributes>& InAsyncOp)
	{
		const FModifyLobbyMemberAttributes::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FLobbyDataEOS>& LobbyData = GetOpDataChecked<TSharedRef<FLobbyDataEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DATA);

		// Update local cache and fire events.
		LobbyData->GetLobbyClientData()->CommitClientChanges({ &LobbyEvents });

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::ModifyLobbyMemberAttributes] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FModifyLobbyMemberAttributes::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetJoinedLobbies> FLobbiesEOSGS::GetJoinedLobbies(FGetJoinedLobbies::Params&& Params)
{
	if (const TSet<TSharedRef<FLobbyDataEOS>>* Lobbies = ActiveLobbies.Find(Params.LocalAccountId))
	{
		FGetJoinedLobbies::Result Result;
		Result.Lobbies.Reserve(Lobbies->Num());
		for (const TSharedRef<FLobbyDataEOS>& LobbyDataEOS : *Lobbies)
		{
			Result.Lobbies.Emplace(LobbyDataEOS->GetLobbyClientData()->GetPublicDataPtr());
		}
		return TOnlineResult<FGetJoinedLobbies>(MoveTemp(Result));
	}
	else
	{
		return TOnlineResult<FGetJoinedLobbies>(Errors::InvalidUser());
	}
}

void FLobbiesEOSGS::HandleLobbyUpdated(const EOS_Lobby_LobbyUpdateReceivedCallbackInfo* Data)
{
	if (TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Data->LobbyId))
	{
		FLobbiesProcessLobbyNotificationImpl::Params Params;
		Params.LobbyData = LobbyData;

		ProcessLobbyNotificationImplOp(MoveTemp(Params))
		.OnComplete([LobbyData](const TOnlineResult<FLobbiesProcessLobbyNotificationImpl>& Result)
		{
			if (Result.IsError())
			{
				// Todo: handle failure to update lobby from snapshot.
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleLobbyUpdated] ProcessLobbyNotificationImplOp Failed: Lobby[%ls], Result[%ls]",
					*LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::HandleLobbyUpdated] Succeeded: Lobby[%ls]",
					*LobbyData->GetLobbyIdString());
			}
		});
	}
}

void FLobbiesEOSGS::HandleLobbyMemberUpdated(const EOS_Lobby_LobbyMemberUpdateReceivedCallbackInfo* Data)
{
	if (TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Data->LobbyId))
	{
		FLobbiesProcessLobbyNotificationImpl::Params Params;
		Params.LobbyData = LobbyData;
		Params.MutatedMembers.Add(Data->TargetUserId);

		ProcessLobbyNotificationImplOp(MoveTemp(Params))
		.OnComplete([LobbyData](const TOnlineResult<FLobbiesProcessLobbyNotificationImpl>& Result)
		{
			if (Result.IsError())
			{
				// Todo: handle failure to update lobby from snapshot.
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleLobbyMemberUpdated] ProcessLobbyNotificationImplOp Failed: Lobby[%ls], Result[%ls]",
					*LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::HandleLobbyMemberUpdated] Succeeded: Lobby[%ls]",
					*LobbyData->GetLobbyIdString());
			}
		});
	}
}

void FLobbiesEOSGS::HandleLobbyMemberStatusReceived(const EOS_Lobby_LobbyMemberStatusReceivedCallbackInfo* Data)
{
	if (TSharedPtr<FLobbyDataEOS> LobbyData = LobbyDataRegistry->Find(Data->LobbyId))
	{
		FLobbiesProcessLobbyNotificationImpl::Params Params;
		Params.LobbyData = LobbyData;

		switch (Data->CurrentStatus)
		{
		case EOS_ELobbyMemberStatus::EOS_LMS_JOINED:
			// Fetch member snapshot on join.
			Params.MutatedMembers.Add(Data->TargetUserId);
			break;

		case EOS_ELobbyMemberStatus::EOS_LMS_PROMOTED:
			// No member data needed, only lobby snapshot.
			break;

		case EOS_ELobbyMemberStatus::EOS_LMS_LEFT:
			Params.LeavingMembers.Add(Data->TargetUserId, ELobbyMemberLeaveReason::Left);
			break;

		case EOS_ELobbyMemberStatus::EOS_LMS_DISCONNECTED:
			Params.LeavingMembers.Add(Data->TargetUserId, ELobbyMemberLeaveReason::Disconnected);
			break;

		case EOS_ELobbyMemberStatus::EOS_LMS_KICKED:
			Params.LeavingMembers.Add(Data->TargetUserId, ELobbyMemberLeaveReason::Kicked);
			break;

		default: checkNoEntry(); // Intentional fallthrough
		case EOS_ELobbyMemberStatus::EOS_LMS_CLOSED:
			Params.LeavingMembers.Add(Data->TargetUserId, ELobbyMemberLeaveReason::Closed);
			break;
		}

		ProcessLobbyNotificationImplOp(MoveTemp(Params))
		.OnComplete([LobbyData](const TOnlineResult<FLobbiesProcessLobbyNotificationImpl>& Result)
		{
			if (Result.IsError())
			{
				// Todo: handle failure to update lobby from snapshot.
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleLobbyMemberStatusReceived] ProcessLobbyNotificationImplOp Failed: Lobby[%ls], Result[%ls]",
					*LobbyData->GetLobbyIdString(), *Result.GetErrorValue().GetLogString());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::HandleLobbyMemberStatusReceived] Succeeded: Lobby[%ls]",
					*LobbyData->GetLobbyIdString());
			}
		});
	}
}

void FLobbiesEOSGS::HandleLobbyInviteReceived(const EOS_Lobby_LobbyInviteReceivedCallbackInfo* Data)
{
	// Todo: Queue this like an operation.
	const FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (LocalAccountId.IsValid())
	{
		FLobbyInviteDataEOS::CreateFromInviteId(LobbyPrerequisites.ToSharedRef(), LobbyDataRegistry.ToSharedRef(), LocalAccountId, Data->InviteId, Data->TargetUserId)
		.Next([this](TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>&& Result)
		{
			if (Result.IsError())
			{
				// Todo: Log / queue a manual fetch of invitations.
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleLobbyInviteReceived] FLobbyInviteDataEOS::CreateFromInviteId Failed: Result[%ls]",
					*Result.GetErrorValue().GetLogString());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::HandleLobbyMemberStatusReceived] Succeeded: InviteId[%ls], Lobby[%ls], Receiver[%ls], Sender[%ls]",
					*Result.GetOkValue()->GetInviteId(),
					*Result.GetOkValue()->GetLobbyData()->GetLobbyIdString(),
					*ToLogString(Result.GetOkValue()->GetReceiver()),
					*ToLogString(Result.GetOkValue()->GetSender()));
				AddActiveInvite(Result.GetOkValue());
			}
		});
	}
}

void FLobbiesEOSGS::HandleLobbyInviteAccepted(const EOS_Lobby_LobbyInviteAcceptedCallbackInfo* Data)
{
	// Todo: Queue this like an operation.
	const FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (LocalAccountId.IsValid())
	{
		FLobbyInviteDataEOS::CreateFromInviteId(LobbyPrerequisites.ToSharedRef(), LobbyDataRegistry.ToSharedRef(), LocalAccountId, Data->InviteId, Data->TargetUserId)
		.Next([this](TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>&& Result)
		{
			if (Result.IsError())
			{
				// Todo: Log / queue a manual fetch of invitations.
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleLobbyInviteAccepted] FLobbyInviteDataEOS::CreateFromInviteId Failed: Result[%ls]",
					*Result.GetErrorValue().GetLogString());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::HandleLobbyInviteAccepted] Succeeded: InviteId[%ls], Lobby[%ls], Receiver[%ls], Sender[%ls]",
					*Result.GetOkValue()->GetInviteId(),
					*Result.GetOkValue()->GetLobbyData()->GetLobbyIdString(),
					*ToLogString(Result.GetOkValue()->GetReceiver()),
					*ToLogString(Result.GetOkValue()->GetSender()));

				const TSharedRef<FLobbyInviteDataEOS>& Invite = Result.GetOkValue();
				LobbyEvents.OnUILobbyJoinRequested.Broadcast(
					FUILobbyJoinRequested{
						Invite->GetReceiver(),
						TResult<TSharedRef<const FLobby>, FOnlineError>(Invite->GetLobbyData()->GetLobbyClientData()->GetPublicDataPtr()),
						EUILobbyJoinRequestedSource::FromInvitation
					});
			}
		});
	}
}

void FLobbiesEOSGS::HandleJoinLobbyAccepted(const EOS_Lobby_JoinLobbyAcceptedCallbackInfo* Data)
{
	// Todo: Queue this like an operation.
	const FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (LocalAccountId.IsValid())
	{
		TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> LobbyDetailsResult = 
			FLobbyDetailsEOS::CreateFromUiEventId(LobbyPrerequisites.ToSharedRef(), LocalAccountId, Data->UiEventId);
		if (LobbyDetailsResult.IsError())
		{
			// Todo: Log / queue a manual fetch of invitations.
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleJoinLobbyAccepted] FLobbyDetailsEOS::CreateFromUiEventId Failed: User[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *LobbyDetailsResult.GetErrorValue().GetLogString());
			return;
		}

		LobbyDataRegistry->FindOrCreateFromLobbyDetails(LocalAccountId, LobbyDetailsResult.GetOkValue())
		.Next([this, LocalAccountId](TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>> && Result)
		{
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleJoinLobbyAccepted] FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails Failed: User[%ls], Result[%ls]",
					*ToLogString(LocalAccountId), *Result.GetErrorValue().GetLogString());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::HandleJoinLobbyAccepted] Succeeded: User[%ls], Lobby[%ls]",
					*ToLogString(LocalAccountId), *Result.GetOkValue()->GetLobbyIdString());

				LobbyEvents.OnUILobbyJoinRequested.Broadcast(
					FUILobbyJoinRequested{
						LocalAccountId,
						TResult<TSharedRef<const FLobby>, FOnlineError>(Result.GetOkValue()->GetLobbyClientData()->GetPublicDataPtr()),
						EUILobbyJoinRequestedSource::Unspecified
					});
			}
		});
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::HandleJoinLobbyAccepted] Failed: Invalid local user.");
	}
}

#if !UE_BUILD_SHIPPING
void FLobbiesEOSGS::CheckMetadata()
{
	// Metadata sanity check.
	ToLogString(FLobbiesLeaveLobbyImpl::Params());
	ToLogString(FLobbiesLeaveLobbyImpl::Result());
	ToLogString(FLobbiesDestroyLobbyImpl::Params());
	ToLogString(FLobbiesDestroyLobbyImpl::Result());
	ToLogString(FLobbiesInviteLobbyMemberImpl::Params());
	ToLogString(FLobbiesInviteLobbyMemberImpl::Result());
	ToLogString(FLobbiesDeclineLobbyInvitationImpl::Params());
	ToLogString(FLobbiesDeclineLobbyInvitationImpl::Result());
	ToLogString(FLobbiesKickLobbyMemberImpl::Params());
	ToLogString(FLobbiesKickLobbyMemberImpl::Result());
	ToLogString(FLobbiesPromoteLobbyMemberImpl::Params());
	ToLogString(FLobbiesPromoteLobbyMemberImpl::Result());
	ToLogString(FLobbiesModifyLobbyDataImpl::Params());
	ToLogString(FLobbiesModifyLobbyDataImpl::Result());
	ToLogString(FLobbiesProcessLobbyNotificationImpl::Params());
	ToLogString(FLobbiesProcessLobbyNotificationImpl::Result());
	Meta::VisitFields(FLobbiesLeaveLobbyImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesLeaveLobbyImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesDestroyLobbyImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesDestroyLobbyImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesInviteLobbyMemberImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesInviteLobbyMemberImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesDeclineLobbyInvitationImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesDeclineLobbyInvitationImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesKickLobbyMemberImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesKickLobbyMemberImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesPromoteLobbyMemberImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesPromoteLobbyMemberImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesModifyLobbyDataImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesModifyLobbyDataImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesProcessLobbyNotificationImpl::Params(), [](const TCHAR* Name, auto& Field) { return false; });
	Meta::VisitFields(FLobbiesProcessLobbyNotificationImpl::Result(), [](const TCHAR* Name, auto& Field) { return false; });
}
#endif

void FLobbiesEOSGS::RegisterHandlers()
{
	// Register for lobby updates.
	OnLobbyUpdatedEOSEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		LobbyPrerequisites->LobbyInterfaceHandle,
		1,
		&EOS_Lobby_AddNotifyLobbyUpdateReceived,
		&EOS_Lobby_RemoveNotifyLobbyUpdateReceived,
		&FLobbiesEOSGS::HandleLobbyUpdated);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYUPDATERECEIVED_API_LATEST, 1);

	// Register for lobby member updates.
	OnLobbyMemberUpdatedEOSEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		LobbyPrerequisites->LobbyInterfaceHandle,
		1,
		&EOS_Lobby_AddNotifyLobbyMemberUpdateReceived,
		&EOS_Lobby_RemoveNotifyLobbyMemberUpdateReceived,
		&FLobbiesEOSGS::HandleLobbyMemberUpdated);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYMEMBERUPDATERECEIVED_API_LATEST, 1);

	// Register for lobby member status changed.
	OnLobbyMemberStatusReceivedEOSEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		LobbyPrerequisites->LobbyInterfaceHandle,
		1,
		&EOS_Lobby_AddNotifyLobbyMemberStatusReceived,
		&EOS_Lobby_RemoveNotifyLobbyMemberStatusReceived,
		&FLobbiesEOSGS::HandleLobbyMemberStatusReceived);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYMEMBERSTATUSRECEIVED_API_LATEST, 1);

	// Register for lobby invite received.
	OnLobbyInviteReceivedEOSEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		LobbyPrerequisites->LobbyInterfaceHandle,
		1,
		&EOS_Lobby_AddNotifyLobbyInviteReceived,
		&EOS_Lobby_RemoveNotifyLobbyInviteReceived,
		&FLobbiesEOSGS::HandleLobbyInviteReceived);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYINVITERECEIVED_API_LATEST, 1);

	// Register for lobby invite accepted.
	OnLobbyInviteAcceptedEOSEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		LobbyPrerequisites->LobbyInterfaceHandle,
		1,
		&EOS_Lobby_AddNotifyLobbyInviteAccepted,
		&EOS_Lobby_RemoveNotifyLobbyInviteAccepted,
		&FLobbiesEOSGS::HandleLobbyInviteAccepted);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYLOBBYINVITEACCEPTED_API_LATEST, 1);

	// Register for join lobby accepted via overlay.
	OnJoinLobbyAcceptedEOSEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		LobbyPrerequisites->LobbyInterfaceHandle,
		1,
		&EOS_Lobby_AddNotifyJoinLobbyAccepted,
		&EOS_Lobby_RemoveNotifyJoinLobbyAccepted,
		&FLobbiesEOSGS::HandleJoinLobbyAccepted);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYJOINLOBBYACCEPTED_API_LATEST, 1);
}

void FLobbiesEOSGS::UnregisterHandlers()
{
	OnLobbyUpdatedEOSEventRegistration = nullptr;
	OnLobbyMemberUpdatedEOSEventRegistration = nullptr;
	OnLobbyMemberStatusReceivedEOSEventRegistration = nullptr;
	OnLobbyInviteReceivedEOSEventRegistration = nullptr;
	OnLobbyInviteAcceptedEOSEventRegistration = nullptr;
	OnJoinLobbyAcceptedEOSEventRegistration = nullptr;
}

void FLobbiesEOSGS::AddActiveLobby(FAccountId LocalAccountId, bool bPresenceEnabled, const TSharedRef<FLobbyDataEOS>& LobbyData)
{
	// Add bookkeeping for the user.
	ActiveLobbies.FindOrAdd(LocalAccountId).Add(LobbyData);

	if (bPresenceEnabled)
	{
		PresenceLobbiesUserMap.FindOrAdd(LocalAccountId, LobbyData->GetLobbyIdHandle());
	}
}

void FLobbiesEOSGS::RemoveActiveLobby(FAccountId LocalAccountId, const TSharedRef<FLobbyDataEOS>& LobbyData)
{
	// Remove bookkeeping for the local user.
	if (TSet<TSharedRef<FLobbyDataEOS>>* Lobbies = ActiveLobbies.Find(LocalAccountId))
	{
		Lobbies->Remove(LobbyData);
	}

	// We'll clear the presence mapping if this was the presence session
	TOnlineResult<FIsPresenceLobby> IsPresenceLobbyResult = IsPresenceLobby({ LocalAccountId, LobbyData->GetLobbyIdHandle() });
	if (IsPresenceLobbyResult.IsOk())
	{
		if (IsPresenceLobbyResult.GetOkValue().bIsPresenceLobby)
		{
			PresenceLobbiesUserMap.Remove(LocalAccountId);
		}
	}
}

void FLobbiesEOSGS::AddActiveInvite(const TSharedRef<FLobbyInviteDataEOS>& Invite)
{
	TMap<FLobbyId, TSharedRef<FLobbyInviteDataEOS>>& ActiveUserInvites = ActiveInvites.FindOrAdd(Invite->GetReceiver());
	const FLobbyId LobbyId = Invite->GetLobbyData()->GetLobbyIdHandle();

	// Todo: Handle multiple invites for the same lobby.
	if (ActiveUserInvites.Find(LobbyId) == nullptr)
	{
		ActiveUserInvites.Add(LobbyId, Invite);
		LobbyEvents.OnLobbyInvitationAdded.Broadcast(
			FLobbyInvitationAdded{
				Invite->GetReceiver(),
				Invite->GetSender(),
				Invite->GetLobbyData()->GetLobbyClientData()->GetPublicDataPtr()
			});
	}
}

void FLobbiesEOSGS::RemoveActiveInvite(const TSharedRef<FLobbyInviteDataEOS>& Invite)
{
	ActiveInvites.FindOrAdd(Invite->GetReceiver()).Remove(Invite->GetLobbyData()->GetLobbyIdHandle());

	LobbyEvents.OnLobbyInvitationRemoved.Broadcast(
		FLobbyInvitationRemoved{
			Invite->GetReceiver(),
			Invite->GetSender(),
			Invite->GetLobbyData()->GetLobbyClientData()->GetPublicDataPtr()
		});
}

TSharedPtr<FLobbyInviteDataEOS> FLobbiesEOSGS::GetActiveInvite(FAccountId TargetUser, FLobbyId TargetLobbyId)
{
	const TSharedRef<FLobbyInviteDataEOS>* Result = ActiveInvites.FindOrAdd(TargetUser).Find(TargetLobbyId);
	return Result ? *Result : TSharedPtr<FLobbyInviteDataEOS>();
}

TFuture<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>> FLobbiesEOSGS::LeaveLobbyImpl(FLobbiesLeaveLobbyImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::LeaveLobbyImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (!Params.LobbyData.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::LeaveLobbyImpl] Failed: No lobby data provided.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	EOS_Lobby_LeaveLobbyOptions LeaveLobbyOptions = {};
	LeaveLobbyOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_LEAVELOBBY_API_LATEST, 1);
	LeaveLobbyOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	LeaveLobbyOptions.LobbyId = Params.LobbyData->GetLobbyIdEOS();

	TPromise<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Lobby_LeaveLobby, LobbyPrerequisites->LobbyInterfaceHandle, LeaveLobbyOptions,
	[
		LocalAccountId = MoveTemp(Params.LocalAccountId),
		LobbyData = MoveTemp(Params.LobbyData),
		Promise = MoveTemp(Promise)
	](const EOS_Lobby_LeaveLobbyCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DestroyLobbyImpl] EOS_Lobby_DestroyLobby Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *LobbyData->GetLobbyIdString(), *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::DestroyLobbyImpl] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *LobbyData->GetLobbyIdString());
		Promise.EmplaceValue(FLobbiesLeaveLobbyImpl::Result{});
	});

	return Future;
}

TFuture<TDefaultErrorResult<FLobbiesDestroyLobbyImpl>> FLobbiesEOSGS::DestroyLobbyImpl(FLobbiesDestroyLobbyImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DestroyLobbyImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesDestroyLobbyImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (Params.LobbyIdString.IsEmpty())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DestroyLobbyImpl] Failed: No lobby id provided.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesDestroyLobbyImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	FTCHARToUTF8 LobbyIdTranslator(*Params.LobbyIdString);
	EOS_Lobby_DestroyLobbyOptions DestroyLobbyOptions = {};
	DestroyLobbyOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_DESTROYLOBBY_API_LATEST, 1);
	DestroyLobbyOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	DestroyLobbyOptions.LobbyId = LobbyIdTranslator.Get();

	TPromise<TDefaultErrorResult<FLobbiesDestroyLobbyImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesDestroyLobbyImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Lobby_DestroyLobby, LobbyPrerequisites->LobbyInterfaceHandle, DestroyLobbyOptions,
	[
		LocalAccountId = MoveTemp(Params.LocalAccountId),
		LobbyIdString = MoveTemp(Params.LobbyIdString),
		Promise = MoveTemp(Promise)
	](const EOS_Lobby_DestroyLobbyCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DestroyLobbyImpl] EOS_Lobby_DestroyLobby Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *LobbyIdString, *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::DestroyLobbyImpl] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *LobbyIdString);
		Promise.EmplaceValue(FLobbiesDestroyLobbyImpl::Result{});
	});

	return Future;
}

TFuture<TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>> FLobbiesEOSGS::InviteLobbyMemberImpl(FLobbiesInviteLobbyMemberImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMemberImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (!Params.LobbyData.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMemberImpl] Failed: No lobby data provided.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	EOS_Lobby_SendInviteOptions SendInviteOptions = {};
	SendInviteOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_SENDINVITE_API_LATEST, 1);
	SendInviteOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	SendInviteOptions.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);
	SendInviteOptions.LobbyId = Params.LobbyData->GetLobbyIdEOS();

	TPromise<TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Lobby_SendInvite, LobbyPrerequisites->LobbyInterfaceHandle, SendInviteOptions,
	[
		LocalAccountId = MoveTemp(Params.LocalAccountId),
		TargetAccountId = MoveTemp(Params.TargetAccountId),
		LobbyData = MoveTemp(Params.LobbyData),
		Promise = MoveTemp(Promise)
	](const EOS_Lobby_SendInviteCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::InviteLobbyMemberImpl] EOS_Lobby_SendInvite Failed: User[%ls], TargetUser[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *ToLogString(TargetAccountId), *LobbyData->GetLobbyIdString(), *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::InviteLobbyMemberImpl] Succeeded: User[%ls], TargetUser[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *ToLogString(TargetAccountId), *LobbyData->GetLobbyIdString());
		Promise.EmplaceValue(FLobbiesInviteLobbyMemberImpl::Result{});
	});

	return Future;
}

TFuture<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>> FLobbiesEOSGS::DeclineLobbyInvitationImpl(FLobbiesDeclineLobbyInvitationImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitationImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitationImpl] Failed: Lobby id is invalid.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Find the active invitation.
	TSharedPtr<FLobbyInviteDataEOS> InviteData = GetActiveInvite(Params.LocalAccountId, Params.LobbyId);
	if (!InviteData)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitationImpl] Failed: Unable to find invitation for lobby. User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *ToLogString(Params.LobbyId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	EOS_Lobby_RejectInviteOptions RejectInviteOptions = {};
	RejectInviteOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_REJECTINVITE_API_LATEST, 1);
	RejectInviteOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	RejectInviteOptions.InviteId = InviteData->GetInviteIdEOS();

	TPromise<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Lobby_RejectInvite, LobbyPrerequisites->LobbyInterfaceHandle, RejectInviteOptions,
	[
		this,
		LocalAccountId = MoveTemp(Params.LocalAccountId),
		InviteData = MoveTemp(InviteData),
		Promise = MoveTemp(Promise)
	](const EOS_Lobby_RejectInviteCallbackInfo* Result) mutable
	{
		// Remove active invitation.
		RemoveActiveInvite(InviteData.ToSharedRef());

		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::DeclineLobbyInvitationImpl] EOS_Lobby_RejectInvite Failed: User[%ls], Invitation[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *InviteData->GetInviteId(), *InviteData->GetLobbyData()->GetLobbyIdString(), *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::DeclineLobbyInvitationImpl] Succeeded: User[%ls], Invitation[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *InviteData->GetInviteId(), *InviteData->GetLobbyData()->GetLobbyIdString());
		Promise.EmplaceValue(FLobbiesDeclineLobbyInvitationImpl::Result{});
	});

	return Future;
}

TFuture<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>> FLobbiesEOSGS::KickLobbyMemberImpl(FLobbiesKickLobbyMemberImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMemberImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (!Params.LobbyData.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMemberImpl] Failed: No lobby data provided.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>>(Errors::InvalidParams()).GetFuture();
	}

	if (Params.LobbyData->GetLobbyClientData()->GetPublicData().OwnerAccountId != Params.LocalAccountId)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMemberImpl] Failed: User is not the lobby owner. User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *Params.LobbyData->GetLobbyIdString());
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	EOS_Lobby_KickMemberOptions KickMemberOptions = {};
	KickMemberOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_KICKMEMBER_API_LATEST, 1);
	KickMemberOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	KickMemberOptions.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);
	KickMemberOptions.LobbyId = Params.LobbyData->GetLobbyIdEOS();

	TPromise<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Lobby_KickMember, LobbyPrerequisites->LobbyInterfaceHandle, KickMemberOptions,
	[
		LocalAccountId = MoveTemp(Params.LocalAccountId),
		TargetAccountId = MoveTemp(Params.TargetAccountId),
		LobbyData = MoveTemp(Params.LobbyData),
		Promise = MoveTemp(Promise)
	](const EOS_Lobby_KickMemberCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::KickLobbyMemberImpl] EOS_Lobby_KickMember Failed: User[%ls], TargetUser[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *ToLogString(TargetAccountId), *LobbyData->GetLobbyIdString(), *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::KickLobbyMemberImpl] Succeeded: User[%ls], TargetUser[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *ToLogString(TargetAccountId), *LobbyData->GetLobbyIdString());
		Promise.EmplaceValue(FLobbiesKickLobbyMemberImpl::Result{});
	});

	return Future;
}

TFuture<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>> FLobbiesEOSGS::PromoteLobbyMemberImpl(FLobbiesPromoteLobbyMemberImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::PromoteLobbyMemberImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (!Params.LobbyData.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::PromoteLobbyMemberImpl] Failed: No lobby data provided.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>>(Errors::InvalidParams()).GetFuture();
	}

	if (Params.LobbyData->GetLobbyClientData()->GetPublicData().OwnerAccountId != Params.LocalAccountId)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::PromoteLobbyMemberImpl] Failed: User is not the lobby owner. User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *Params.LobbyData->GetLobbyIdString());
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	EOS_Lobby_PromoteMemberOptions PromoteMemberOptions = {};
	PromoteMemberOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_PROMOTEMEMBER_API_LATEST, 1);
	PromoteMemberOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	PromoteMemberOptions.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);
	PromoteMemberOptions.LobbyId = Params.LobbyData->GetLobbyIdEOS();

	TPromise<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>> Future = Promise.GetFuture();

	EOS_Async(EOS_Lobby_PromoteMember, LobbyPrerequisites->LobbyInterfaceHandle, PromoteMemberOptions,
	[
		LocalAccountId = MoveTemp(Params.LocalAccountId),
		TargetAccountId = MoveTemp(Params.TargetAccountId),
		LobbyData = MoveTemp(Params.LobbyData),
		Promise = MoveTemp(Promise)
	](const EOS_Lobby_PromoteMemberCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::PromoteLobbyMemberImpl] EOS_Lobby_PromoteMember Failed: User[%ls], TargetUser[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *ToLogString(TargetAccountId), *LobbyData->GetLobbyIdString(), *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::PromoteLobbyMemberImpl] Succeeded: User[%ls], TargetUser[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *ToLogString(TargetAccountId), *LobbyData->GetLobbyIdString());
		Promise.EmplaceValue(FLobbiesPromoteLobbyMemberImpl::Result{});
	});

	return Future;
}

TFuture<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>> FLobbiesEOSGS::ModifyLobbyDataImpl(FLobbiesModifyLobbyDataImpl::Params&& Params)
{
	// Check prerequisites.
	if (!Services.GetAuthInterface()->IsLoggedIn(Params.LocalAccountId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyDataImpl] Failed: User is not logged in. User[%ls]", *ToLogString(Params.LocalAccountId));
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(Errors::InvalidUser()).GetFuture();
	}

	if (!Params.LobbyData.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyDataImpl] Failed: No lobby data provided.");
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(Errors::InvalidParams()).GetFuture();
	}

	TSharedPtr<FLobbyDetailsEOS> LobbyDetails = Params.LobbyData->GetUserLobbyDetails(Params.LocalAccountId);
	if (!LobbyDetails)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyDataImpl] Failed: Unable to find lobby details for user. User[%ls], Lobby[%ls]",
			*ToLogString(Params.LocalAccountId), *Params.LobbyData->GetLobbyIdString());
		return MakeFulfilledPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>>(Errors::InvalidParams()).GetFuture();
	}

	// Start operation.
	TPromise<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>> Promise;
	TFuture<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>> Future = Promise.GetFuture();

	LobbyDetails->ApplyLobbyDataUpdateFromLocalChanges(Params.LocalAccountId, Params.ServiceChanges)
	.Next([
		LocalAccountId = Params.LocalAccountId,
		LobbyData = Params.LobbyData,
		Promise = MoveTemp(Promise)
	](EOS_EResult EosResult) mutable
	{
		if (EosResult != EOS_EResult::EOS_Success && EosResult != EOS_EResult::EOS_NoChange)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ModifyLobbyDataImpl] ApplyLobbyDataUpdateFromLocalChanges Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *LobbyData->GetLobbyIdString(), *LexToString(EosResult));
			Promise.EmplaceValue(Errors::FromEOSResult(EosResult));
			return;
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::ModifyLobbyDataImpl] Succeeded: User[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *LobbyData->GetLobbyIdString());
		Promise.EmplaceValue(FLobbiesModifyLobbyDataImpl::Result{});
	});

	return Future;
}

TOnlineAsyncOpHandle<FLobbiesProcessLobbyNotificationImpl> FLobbiesEOSGS::ProcessLobbyNotificationImplOp(FLobbiesProcessLobbyNotificationImpl::Params&& InParams)
{
	TOnlineAsyncOpRef<FLobbiesProcessLobbyNotificationImpl> Op = GetOp<FLobbiesProcessLobbyNotificationImpl>(MoveTemp(InParams));
	const FLobbiesProcessLobbyNotificationImpl::Params& Params = Op->GetParams();

	// Check prerequisites.
	if (!Params.LobbyData.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ProcessLobbyNotificationImplOp] Failed: No lobby data provided.");
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	// Start operation.
	Op->Then([this](TOnlineAsyncOp<FLobbiesProcessLobbyNotificationImpl>& InAsyncOp)
	{
		const FLobbiesProcessLobbyNotificationImpl::Params& Params = InAsyncOp.GetParams();

		// Notifications do not always indicate a user. Try to find a valid lobby details object to
		// handle acquiring data snapshots.
		TSharedPtr<FLobbyDetailsEOS> LobbyDetails = Params.LobbyData->GetActiveLobbyDetails();
		if (!LobbyDetails.IsValid())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ProcessLobbyNotificationImplOp] Failed: Unable to find active lobby details to process notificaions. Lobby[%ls]",
				*Params.LobbyData->GetLobbyIdString());
			InAsyncOp.SetError(Errors::InvalidState());
			return MakeFulfilledPromise<TDefaultErrorResultInternal<FLobbyServiceSnapshot>>(Errors::InvalidState()).GetFuture();
		}

		InAsyncOp.Data.Set<TSharedRef<FLobbyDetailsEOS>>(UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS, LobbyDetails.ToSharedRef());

		// Fetch lobby snapshot. Fetching the snapshot resolves the account ids of all lobby members in the snapshot.
		return LobbyDetails->GetLobbySnapshot(true);
	})
	.Then([this](TOnlineAsyncOp<FLobbiesProcessLobbyNotificationImpl>& InAsyncOp, TDefaultErrorResultInternal<FLobbyServiceSnapshot>&& LobbySnapshotResult)
	{
		const FLobbiesProcessLobbyNotificationImpl::Params& Params = InAsyncOp.GetParams();
		TSharedRef<FLobbyDetailsEOS> LobbyDetails = GetOpDataChecked<TSharedRef<FLobbyDetailsEOS>>(InAsyncOp, UE_ONLINE_LOBBY_EOS_KEY_NAME_LOBBY_DETAILS);

		if (LobbySnapshotResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ProcessLobbyNotificationImplOp] GetLobbySnapshot Failed. Lobby[%ls], Result[%ls]",
				*Params.LobbyData->GetLobbyIdString(), *LobbySnapshotResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(LobbySnapshotResult.GetErrorValue()));
			return;
		}

		// Get member snapshots.
		TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
		LobbyMemberSnapshots.Reserve(Params.MutatedMembers.Num());
		for (EOS_ProductUserId MutatedMember : Params.MutatedMembers)
		{
			const FAccountId MutatedMemberAccountId = FindAccountId(MutatedMember);
			if (MutatedMemberAccountId.IsValid())
			{
				TDefaultErrorResultInternal<FLobbyMemberServiceSnapshot> MemberSnapshotResult = LobbyDetails->GetLobbyMemberSnapshot(MutatedMemberAccountId);
				if (MemberSnapshotResult.IsError())
				{
					UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ProcessLobbyNotificationImplOp] GetLobbyMemberSnapshot Failed. User[%ls], Lobby[%ls], Result[%ls]",
						*ToLogString(MutatedMemberAccountId), *Params.LobbyData->GetLobbyIdString(), *LobbySnapshotResult.GetErrorValue().GetLogString());
					InAsyncOp.SetError(MoveTemp(MemberSnapshotResult.GetErrorValue()));
					return;
				}
				LobbyMemberSnapshots.Add(MutatedMemberAccountId, MoveTemp(MemberSnapshotResult.GetOkValue()));
			}
		}

		// Translate leaving members from EOS_ProductUserId to FAccountId.
		TMap<FAccountId, ELobbyMemberLeaveReason> LeavingMemberReason;
		LeavingMemberReason.Reserve(Params.LeavingMembers.Num());
		for (const TPair<EOS_ProductUserId, ELobbyMemberLeaveReason>& LeavingMember : Params.LeavingMembers)
		{
			const FAccountId LeavingMemberAccountId = FindAccountId(LeavingMember.Key);
			if (LeavingMemberAccountId.IsValid())
			{
				LeavingMemberReason.Add(LeavingMemberAccountId, LeavingMember.Value);
			}
		}

		// Prepare client state from snapshot.
		TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareSnapshotResult =
			Params.LobbyData->GetLobbyClientData()->PrepareServiceSnapshot({ MoveTemp(LobbySnapshotResult.GetOkValue()), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeavingMemberReason) });
		if (PrepareSnapshotResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbiesEOSGS::ProcessLobbyNotificationImplOp] PrepareServiceSnapshot Failed. Lobby[%ls], Result[%ls]",
				*Params.LobbyData->GetLobbyIdString(), *PrepareSnapshotResult.GetErrorValue().GetLogString());
			InAsyncOp.SetError(MoveTemp(PrepareSnapshotResult.GetErrorValue()));
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FLobbiesProcessLobbyNotificationImpl>& InAsyncOp)
	{
		const FLobbiesProcessLobbyNotificationImpl::Params& Params = InAsyncOp.GetParams();

		// Apply updates and fire notifications.
		FLobbyClientDataCommitServiceSnapshot::Result Result = Params.LobbyData->GetLobbyClientData()->CommitServiceSnapshot({ &LobbyEvents });

		// Remove active users if needed.
		for (FAccountId LeavingMember : Result.LeavingLocalMembers)
		{
			RemoveActiveLobby(LeavingMember, Params.LobbyData.ToSharedRef());
		}

		UE_LOGF(LogOnlineServices, Verbose, "[FLobbiesEOSGS::ProcessLobbyNotificationImplOp] Succeeded: Lobby[%ls]",
			*Params.LobbyData->GetLobbyIdString());
		InAsyncOp.SetResult(FLobbiesProcessLobbyNotificationImpl::Result{});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

FAccountId FLobbiesEOSGS::FindAccountId(const EOS_ProductUserId ProductUserId)
{
	return UE::Online::FindAccountId(Services.GetServicesProvider(), ProductUserId);
}

/* UE::Online */ }
