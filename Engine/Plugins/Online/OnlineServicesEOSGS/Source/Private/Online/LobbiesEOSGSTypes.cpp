// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbiesEOSGSTypes.h"

#include "EOSShared.h"
#include "EOSSharedTypes.h"
#include "Online/EpicProductUserIdResolver.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesLog.h"
#include "Online/OnlineUtils.h"
#include <eos_lobby.h>

namespace UE::Online {

namespace Private {

EOS_EComparisonOp TranslateSearchComparison(ESchemaAttributeComparisonOp Op)
{
	switch (Op)
	{
	default:												checkNoEntry(); // Intentional fallthrough
	case ESchemaAttributeComparisonOp::Equals:				return EOS_EComparisonOp::EOS_CO_EQUAL;
	case ESchemaAttributeComparisonOp::NotEquals:			return EOS_EComparisonOp::EOS_CO_NOTEQUAL;
	case ESchemaAttributeComparisonOp::GreaterThan:			return EOS_EComparisonOp::EOS_CO_GREATERTHAN;
	case ESchemaAttributeComparisonOp::GreaterThanEquals:	return EOS_EComparisonOp::EOS_CO_GREATERTHANOREQUAL;
	case ESchemaAttributeComparisonOp::LessThan:			return EOS_EComparisonOp::EOS_CO_LESSTHAN;
	case ESchemaAttributeComparisonOp::LessThanEquals:		return EOS_EComparisonOp::EOS_CO_LESSTHANOREQUAL;
	case ESchemaAttributeComparisonOp::Near:				return EOS_EComparisonOp::EOS_CO_DISTANCE;
	case ESchemaAttributeComparisonOp::In:					return EOS_EComparisonOp::EOS_CO_ONEOF;
	case ESchemaAttributeComparisonOp::NotIn:				return EOS_EComparisonOp::EOS_CO_NOTANYOF;

	// todo:
	// EOS_EComparisonOp::EOS_CO_ANYOF
	// EOS_EComparisonOp::EOS_CO_NOTONEOF
	// EOS_EComparisonOp::EOS_CO_CONTAINS
	}
}

} // Private

FString ToLogString(const FLobbyBucketIdEOS& BucketId)
{
	return FString::Printf(TEXT("%s:%d"), *BucketId.GetProductName(), BucketId.GetProductVersion());
}

FLobbyBucketIdEOS::FLobbyBucketIdEOS(const FString& InProductName, int32 InProductVersion)
	: ProductName(InProductName.Replace(UE_ONLINE_LOBBIES_BUCKET_ID_EOS_SEPARATOR, TEXT("_")))
	, ProductVersion(InProductVersion)
{
}

// Attribute translators.
FLobbyAttributeTranslator<ELobbyTranslationType::ToService>::FLobbyAttributeTranslator(const FSchemaServiceAttributeData& FromAttributeData)
	: KeyConverterStorage(*FromAttributeData.Id.ToString().ToLower())
	, AttributeVisibility(EOS_ELobbyAttributeVisibility::EOS_LAT_PRIVATE)
{
	AttributeData.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);
	AttributeData.Key = KeyConverterStorage.Get();

	switch (FromAttributeData.Value.GetType())
	{
	case ESchemaAttributeType::String:
		ValueConverterStorage.Emplace(*FromAttributeData.Value.GetString());
		AttributeData.ValueType = EOS_ELobbyAttributeType::EOS_AT_STRING;
		AttributeData.Value.AsUtf8 = ValueConverterStorage->Get();
		break;

	case ESchemaAttributeType::Int64:
		AttributeData.ValueType = EOS_ELobbyAttributeType::EOS_AT_INT64;
		AttributeData.Value.AsInt64 = FromAttributeData.Value.GetInt64();
		break;

	case ESchemaAttributeType::Double:
		AttributeData.ValueType = EOS_ELobbyAttributeType::EOS_AT_DOUBLE;
		AttributeData.Value.AsDouble = FromAttributeData.Value.GetDouble();
		break;

	case ESchemaAttributeType::Bool:
		AttributeData.ValueType = EOS_ELobbyAttributeType::EOS_AT_BOOLEAN;
		AttributeData.Value.AsBool = FromAttributeData.Value.GetBoolean();
		break;

	case ESchemaAttributeType::None:
	default:
		checkNoEntry();
	}

	if (EnumHasAnyFlags(FromAttributeData.Flags, ESchemaServiceAttributeFlags::Public))
	{
		AttributeVisibility = EOS_ELobbyAttributeVisibility::EOS_LAT_PUBLIC;
	}
}

FLobbyAttributeTranslator<ELobbyTranslationType::FromService>::FLobbyAttributeTranslator(const EOS_Lobby_AttributeData& FromAttributeData)
{
	FSchemaAttributeId AttributeId(UTF8_TO_TCHAR(FromAttributeData.Key));
	FSchemaVariant VariantData;

	switch (FromAttributeData.ValueType)
	{
	case EOS_ELobbyAttributeType::EOS_AT_BOOLEAN:
		VariantData.Set(FromAttributeData.Value.AsBool != 0);
		break;

	case EOS_ELobbyAttributeType::EOS_AT_INT64:
		VariantData.Set(static_cast<int64>(FromAttributeData.Value.AsInt64));
		break;

	case EOS_ELobbyAttributeType::EOS_AT_DOUBLE:
		VariantData.Set(FromAttributeData.Value.AsDouble);
		break;

	case EOS_ELobbyAttributeType::EOS_AT_STRING:
		VariantData.Set(UTF8_TO_TCHAR(FromAttributeData.Value.AsUtf8));
		break;

	default:
		checkNoEntry();
		break;
	}

	AttributeData = TPair<FSchemaAttributeId, FSchemaVariant>(MoveTemp(AttributeId), MoveTemp(VariantData));
}

FLobbyBucketIdTranslator<ELobbyTranslationType::ToService>::FLobbyBucketIdTranslator(const FLobbyBucketIdEOS& BucketId)
	: BucketConverterStorage(*FString::Printf(TEXT("%s%s%d"), *BucketId.GetProductName(), UE_ONLINE_LOBBIES_BUCKET_ID_EOS_SEPARATOR, BucketId.GetProductVersion()))
{
}

FLobbyBucketIdTranslator<ELobbyTranslationType::FromService>::FLobbyBucketIdTranslator(const char* BucketIdEOS)
{
	FUTF8ToTCHAR BucketConverterStorage(BucketIdEOS);
	FString BucketString(BucketConverterStorage.Get());

	constexpr int32 ExpectedPartsNum = 2;
	TArray<FString> Parts;
	if (BucketString.ParseIntoArray(Parts, UE_ONLINE_LOBBIES_BUCKET_ID_EOS_SEPARATOR) == ExpectedPartsNum)
	{
		int32 BuildId = 0;
		::LexFromString(BuildId, *Parts[1]);
		BucketId = FLobbyBucketIdEOS(Parts[0], BuildId);
	}
}

const EOS_HLobbyDetails FLobbyDetailsEOS::InvalidLobbyDetailsHandle = {};

TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> FLobbyDetailsEOS::CreateFromLobbyId(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	FAccountId LocalAccountId,
	EOS_LobbyId LobbyIdEOS)
{
	EOS_HLobbyDetails LobbyDetailsHandle = {};

	EOS_Lobby_CopyLobbyDetailsHandleOptions Options;
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLE_API_LATEST, 1);
	Options.LobbyId = LobbyIdEOS;
	Options.LocalUserId = GetProductUserIdChecked(LocalAccountId);

	EOS_EResult EOSResult = EOS_Lobby_CopyLobbyDetailsHandle(Prerequisites->LobbyInterfaceHandle, &Options, &LobbyDetailsHandle);
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromLobbyId] EOS_Lobby_CopyLobbyDetailsHandle Failed: User[%ls], Lobby[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyIdEOS), *ToLogString(EOSResult));
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(Errors::FromEOSResult(EOSResult));
	}

	TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>> LobbyDetailsInfoResult = FLobbyDetailsInfoEOS::Create(LobbyDetailsHandle);
	if (LobbyDetailsInfoResult.IsError())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromLobbyId] FLobbyDetailsInfoEOS::Create Failed: User[%ls], Lobby[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyIdEOS), *ToLogString(EOSResult));
		EOS_LobbyDetails_Release(LobbyDetailsHandle);
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MoveTemp(LobbyDetailsInfoResult.GetErrorValue()));
	}

	UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::CreateFromLobbyId] Succeeded: User[%ls], Lobby[%ls]",
		*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyIdEOS));
	return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MakeShared<FLobbyDetailsEOS>(Prerequisites, LobbyDetailsInfoResult.GetOkValue(), LocalAccountId, ELobbyDetailsSource::Active, LobbyDetailsHandle));
}

TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> FLobbyDetailsEOS::CreateFromLobbyId(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	FAccountId LocalAccountId,
	const FString& LobbyId)
{
	return CreateFromLobbyId(Prerequisites, LocalAccountId, TCHAR_TO_UTF8(*LobbyId));
}

TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> FLobbyDetailsEOS::CreateFromInviteId(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	FAccountId LocalAccountId,
	const char* InviteId)
{
	EOS_HLobbyDetails LobbyDetailsHandle = {};

	EOS_Lobby_CopyLobbyDetailsHandleByInviteIdOptions Options;
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLEBYINVITEID_API_LATEST, 1);
	Options.InviteId = InviteId;

	EOS_EResult EOSResult = EOS_Lobby_CopyLobbyDetailsHandleByInviteId(Prerequisites->LobbyInterfaceHandle, &Options, &LobbyDetailsHandle);
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromInviteId] EOS_Lobby_CopyLobbyDetailsHandleByInviteId Failed: User[%ls], InviteId[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), UTF8_TO_TCHAR(InviteId), *ToLogString(EOSResult));
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(Errors::FromEOSResult(EOSResult));
	}

	TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>> LobbyDetailsInfoResult = FLobbyDetailsInfoEOS::Create(LobbyDetailsHandle);
	if (LobbyDetailsInfoResult.IsError())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromInviteId] FLobbyDetailsInfoEOS::Create Failed: User[%ls], InviteId[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), UTF8_TO_TCHAR(InviteId), *LobbyDetailsInfoResult.GetErrorValue().GetLogString());
		EOS_LobbyDetails_Release(LobbyDetailsHandle);
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MoveTemp(LobbyDetailsInfoResult.GetErrorValue()));
	}

	UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::CreateFromInviteId] Succeeded: User[%ls], InviteId[%ls], Lobby[%ls]",
		*ToLogString(LocalAccountId), UTF8_TO_TCHAR(InviteId), UTF8_TO_TCHAR(LobbyDetailsInfoResult.GetOkValue()->GetLobbyIdEOS()));
	return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MakeShared<FLobbyDetailsEOS>(Prerequisites, LobbyDetailsInfoResult.GetOkValue(), LocalAccountId, ELobbyDetailsSource::Invite, LobbyDetailsHandle));
}

TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> FLobbyDetailsEOS::CreateFromUiEventId(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	FAccountId LocalAccountId,
	EOS_UI_EventId UiEventId)
{
	EOS_HLobbyDetails LobbyDetailsHandle = {};

	EOS_Lobby_CopyLobbyDetailsHandleByUiEventIdOptions Options;
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_COPYLOBBYDETAILSHANDLEBYUIEVENTID_API_LATEST, 1);
	Options.UiEventId = UiEventId;

	EOS_EResult EOSResult = EOS_Lobby_CopyLobbyDetailsHandleByUiEventId(Prerequisites->LobbyInterfaceHandle, &Options, &LobbyDetailsHandle);
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromUiEventId] EOS_Lobby_CopyLobbyDetailsHandleByUiEventId Failed: User[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), *ToLogString(EOSResult));
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(Errors::FromEOSResult(EOSResult));
	}

	TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>> LobbyDetailsInfoResult = FLobbyDetailsInfoEOS::Create(LobbyDetailsHandle);
	if (LobbyDetailsInfoResult.IsError())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromUiEventId] FLobbyDetailsInfoEOS::Create Failed: User[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), *LobbyDetailsInfoResult.GetErrorValue().GetLogString());
		EOS_LobbyDetails_Release(LobbyDetailsHandle);
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MoveTemp(LobbyDetailsInfoResult.GetErrorValue()));
	}

	UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::CreateFromUiEventId] Succeeded: User[%ls], Lobby[%ls]",
		*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyDetailsInfoResult.GetOkValue()->GetLobbyIdEOS()));
	return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MakeShared<FLobbyDetailsEOS>(Prerequisites, LobbyDetailsInfoResult.GetOkValue(), LocalAccountId, ELobbyDetailsSource::UiEvent, LobbyDetailsHandle));
}

TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> FLobbyDetailsEOS::CreateFromSearchResult(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	FAccountId LocalAccountId,
	EOS_HLobbySearch SearchHandle,
	uint32_t ResultIndex)
{
	EOS_HLobbyDetails LobbyDetailsHandle = {};

	EOS_LobbySearch_CopySearchResultByIndexOptions Options;
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_COPYSEARCHRESULTBYINDEX_API_LATEST, 1);
	Options.LobbyIndex = ResultIndex;

	EOS_EResult EOSResult = EOS_LobbySearch_CopySearchResultByIndex(SearchHandle, &Options, &LobbyDetailsHandle);
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromSearchResult] EOS_LobbySearch_CopySearchResultByIndex Failed: User[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), *ToLogString(EOSResult));
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(Errors::FromEOSResult(EOSResult));
	}

	TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>> LobbyDetailsInfoResult = FLobbyDetailsInfoEOS::Create(LobbyDetailsHandle);
	if (LobbyDetailsInfoResult.IsError())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::CreateFromSearchResult] FLobbyDetailsInfoEOS::Create Failed: User[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), *LobbyDetailsInfoResult.GetErrorValue().GetLogString());
		EOS_LobbyDetails_Release(LobbyDetailsHandle);
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MoveTemp(LobbyDetailsInfoResult.GetErrorValue()));
	}

	UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::CreateFromSearchResult] Succeeded: User[%ls], Lobby[%ls]",
		*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyDetailsInfoResult.GetOkValue()->GetLobbyIdEOS()));
	return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>>(MakeShared<FLobbyDetailsEOS>(Prerequisites, LobbyDetailsInfoResult.GetOkValue(), LocalAccountId, ELobbyDetailsSource::Search, LobbyDetailsHandle));
}

FLobbyDetailsEOS::~FLobbyDetailsEOS()
{
	EOS_LobbyDetails_Release(LobbyDetailsHandle);
}

bool FLobbyDetailsEOS::IsBucketCompatible() const
{
	return LobbyDetailsInfo->GetBucketId() == Prerequisites->BucketId;
}

TFuture<TDefaultErrorResultInternal<FLobbyServiceSnapshot>> FLobbyDetailsEOS::GetLobbySnapshot(bool bCopyMemberData) const
{
	if (!Prerequisites->ResolverInterface)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::GetLobbySnapshot] Failed: Resolver interface missing. Lobby[%ls]",
			UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()));
		return MakeFulfilledPromise<TDefaultErrorResultInternal<FLobbyServiceSnapshot>>(Errors::MissingInterface()).GetFuture();
	}

	TSharedRef<TArray<EOS_ProductUserId>> MemberProductUserIds = MakeShared<TArray<EOS_ProductUserId>>();

	if(bCopyMemberData)
	{
		EOS_LobbyDetails_GetMemberCountOptions GetMemberCountOptions = {};
		GetMemberCountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETMEMBERCOUNT_API_LATEST, 1);

		const uint32_t MemberCount = EOS_LobbyDetails_GetMemberCount(LobbyDetailsHandle, &GetMemberCountOptions);
		MemberProductUserIds->Reserve(MemberCount);

		for (uint32_t MemberIndex = 0; MemberIndex < MemberCount; ++MemberIndex)
		{
			EOS_LobbyDetails_GetMemberByIndexOptions GetMemberByIndexOptions = {};
			GetMemberByIndexOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETMEMBERBYINDEX_API_LATEST, 1);
			GetMemberByIndexOptions.MemberIndex = MemberIndex;

			// This method will return null if the querying user is not part of the lobby, in which case we can stop iterating
			EOS_ProductUserId MemberId = EOS_LobbyDetails_GetMemberByIndex(LobbyDetailsHandle, &GetMemberByIndexOptions);
			if (MemberId != nullptr)
			{
				MemberProductUserIds->Emplace(MemberId);
			}
			else
			{
				break;
			}
		}
	}

	EOS_LobbyDetails_GetLobbyOwnerOptions GetLobbyOwnerOptions = {};
	GetLobbyOwnerOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETLOBBYOWNER_API_LATEST, 1);

	const EOS_ProductUserId LobbyOwner = EOS_LobbyDetails_GetLobbyOwner(LobbyDetailsHandle, &GetLobbyOwnerOptions);

	// In the case where the querying user is not part of the lobby and no members can be retrieved, we still want to at least resolve the id of the lobby owner, which we can retrieve at all times
	if (MemberProductUserIds->IsEmpty())
	{
		MemberProductUserIds->Emplace(LobbyOwner);
	}

	TPromise<TDefaultErrorResultInternal<FLobbyServiceSnapshot>> Promise;
	TFuture<TDefaultErrorResultInternal<FLobbyServiceSnapshot>> Future = Promise.GetFuture();

	// Resolve lobby member product user ids to FAccountId before proceeding.
	Prerequisites->ResolverInterface->ResolveAccountIds(AssociatedLocalUser, *MemberProductUserIds)
	.Next([StrongThis = AsShared(), Promise = MoveTemp(Promise), MemberProductUserIds, LobbyOwner](TArray<FAccountId>&& ResolvedAccountIds) mutable
	{
		if (MemberProductUserIds->Num() != ResolvedAccountIds.Num())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::GetLobbySnapshot] ResolveAccountIds Failed: Expected Count[%d], Received Count[%d], Lobby[%ls]",
				MemberProductUserIds->Num(), ResolvedAccountIds.Num(), UTF8_TO_TCHAR(StrongThis->GetInfo()->GetLobbyIdEOS()));
			Promise.EmplaceValue(Errors::Unknown());
			return;
		}

		FLobbyServiceSnapshot LobbyServiceSnapshot;
		LobbyServiceSnapshot.MaxMembers = StrongThis->GetInfo()->GetMaxMembers();
		LobbyServiceSnapshot.JoinPolicy = TranslateJoinPolicy(StrongThis->GetInfo()->GetPermissionLevel());

		// Resolve member info.
		{
			for (int32 MemberIndex = 0; MemberIndex < MemberProductUserIds->Num(); ++MemberIndex)
			{
				const EOS_ProductUserId MemberProductUserId = (*MemberProductUserIds)[MemberIndex];
				const FAccountId ResolvedMemberAccountId = ResolvedAccountIds[MemberIndex];

				if (MemberProductUserId == LobbyOwner)
				{
					LobbyServiceSnapshot.OwnerAccountId = ResolvedMemberAccountId;
				}

				LobbyServiceSnapshot.Members.Add(ResolvedMemberAccountId);
			}
		}

		// Resolve lobby attributes
		{
			EOS_LobbyDetails_GetAttributeCountOptions GetAttributeCountOptions = {};
			GetAttributeCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETATTRIBUTECOUNT_API_LATEST, 1);

			const uint32_t AttributeCount = EOS_LobbyDetails_GetAttributeCount(StrongThis->LobbyDetailsHandle, &GetAttributeCountOptions);
			for (uint32_t AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
			{
				EOS_LobbyDetails_CopyAttributeByIndexOptions CopyAttributeByIndexOptions = {};
				CopyAttributeByIndexOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYATTRIBUTEBYINDEX_API_LATEST, 1);
				CopyAttributeByIndexOptions.AttrIndex = AttributeIndex;

				EOS_Lobby_Attribute* LobbyAttribute = nullptr;
				ON_SCOPE_EXIT
				{
					EOS_Lobby_Attribute_Release(LobbyAttribute);
				};

				EOS_EResult EOSResult = EOS_LobbyDetails_CopyAttributeByIndex(StrongThis->LobbyDetailsHandle, &CopyAttributeByIndexOptions, &LobbyAttribute);
				if (EOSResult != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::GetLobbySnapshot] EOS_LobbyDetails_CopyAttributeByIndex Failed: Lobby[%ls], Result[%ls]",
						UTF8_TO_TCHAR(StrongThis->GetInfo()->GetLobbyIdEOS()), *LexToString(EOSResult));
					Promise.EmplaceValue(Errors::FromEOSResult(EOSResult));
					return;
				}

				FLobbyAttributeTranslator<ELobbyTranslationType::FromService> AttributeTranslator(*LobbyAttribute->Data);
				LobbyServiceSnapshot.SchemaServiceSnapshot.Attributes.Add(MoveTemp(AttributeTranslator.GetMutableAttributeData()));
			}
		}

		UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::GetLobbySnapshot] Succeeded: Lobby[%ls]",
			UTF8_TO_TCHAR(StrongThis->GetInfo()->GetLobbyIdEOS()));
		Promise.EmplaceValue(MoveTemp(LobbyServiceSnapshot));
	});

	return Future;
}

TDefaultErrorResultInternal<FLobbyMemberServiceSnapshot> FLobbyDetailsEOS::GetLobbyMemberSnapshot(FAccountId MemberAccountId) const
{
	EOS_ProductUserId MemberProductUserId = GetProductUserIdChecked(MemberAccountId);

	FLobbyMemberServiceSnapshot LobbyMemberServcieSnapshot;
	LobbyMemberServcieSnapshot.AccountId = MemberAccountId;

	// Fetch attributes.
	{
		EOS_LobbyDetails_GetMemberAttributeCountOptions GetMemberAttributeCountOptions = {};
		GetMemberAttributeCountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_GETMEMBERATTRIBUTECOUNT_API_LATEST, 1);
		GetMemberAttributeCountOptions.TargetUserId = MemberProductUserId;

		const uint32_t MemberAttributeCount = EOS_LobbyDetails_GetMemberAttributeCount(LobbyDetailsHandle, &GetMemberAttributeCountOptions);
		for (uint32_t MemberAttributeIndex = 0; MemberAttributeIndex < MemberAttributeCount; ++MemberAttributeIndex)
		{
			EOS_LobbyDetails_CopyMemberAttributeByIndexOptions CopyMemberAttributeByIndexOptions = {};
			CopyMemberAttributeByIndexOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYMEMBERATTRIBUTEBYINDEX_API_LATEST, 1);
			CopyMemberAttributeByIndexOptions.TargetUserId = MemberProductUserId;
			CopyMemberAttributeByIndexOptions.AttrIndex = MemberAttributeIndex;

			EOS_Lobby_Attribute* LobbyAttribute = nullptr;
			ON_SCOPE_EXIT
			{
				EOS_Lobby_Attribute_Release(LobbyAttribute);
			};

			EOS_EResult EOSResult = EOS_LobbyDetails_CopyMemberAttributeByIndex(LobbyDetailsHandle, &CopyMemberAttributeByIndexOptions, &LobbyAttribute);
			if (EOSResult != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::GetLobbyMemberSnapshot] EOS_LobbyDetails_CopyMemberAttributeByIndex Failed: Member[%ls], Lobby[%ls], Result[%ls]",
					*ToLogString(MemberAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *LexToString(EOSResult));
				return TDefaultErrorResultInternal<FLobbyMemberServiceSnapshot>(Errors::FromEOSResult(EOSResult));
			}

			FLobbyAttributeTranslator<ELobbyTranslationType::FromService> AttributeTranslator(*LobbyAttribute->Data);
			LobbyMemberServcieSnapshot.SchemaServiceSnapshot.Attributes.Add(MoveTemp(AttributeTranslator.GetMutableAttributeData()));
		}
	}

	UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::GetLobbyMemberSnapshot] Succeeded: Member[%ls], Lobby[%ls]",
		*ToLogString(MemberAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()));
	return TDefaultErrorResultInternal<FLobbyMemberServiceSnapshot>(MoveTemp(LobbyMemberServcieSnapshot));
}

TFuture<EOS_EResult> FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges(FAccountId LocalAccountId, const FLobbyClientServiceChanges& ServiceChanges) const
{
	EOS_HLobbyModification LobbyModificationHandle = nullptr;

	ON_SCOPE_EXIT
	{
		EOS_LobbyModification_Release(LobbyModificationHandle);
	};

	// Create lobby modification handle.
	EOS_Lobby_UpdateLobbyModificationOptions ModificationOptions = {};
	ModificationOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_UPDATELOBBYMODIFICATION_API_LATEST, 1);
	ModificationOptions.LocalUserId = GetProductUserIdChecked(LocalAccountId);
	ModificationOptions.LobbyId = GetInfo()->GetLobbyIdEOS();

	EOS_EResult EOSResult = EOS_Lobby_UpdateLobbyModification(Prerequisites->LobbyInterfaceHandle, &ModificationOptions, &LobbyModificationHandle);
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges] EOS_Lobby_UpdateLobbyModification Failed: User[%ls], Lobby[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *LexToString(EOSResult));
		return MakeFulfilledPromise<EOS_EResult>(EOSResult).GetFuture();
	}

	// Set lobby join policy.
	if (ServiceChanges.JoinPolicy)
	{
		EOS_LobbyModification_SetPermissionLevelOptions SetPermissionOptions = {};
		SetPermissionOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_SETPERMISSIONLEVEL_API_LATEST, 1);
		SetPermissionOptions.PermissionLevel = TranslateJoinPolicy(*ServiceChanges.JoinPolicy);

		EOSResult = EOS_LobbyModification_SetPermissionLevel(LobbyModificationHandle, &SetPermissionOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges] EOS_LobbyModification_SetPermissionLevel Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *LexToString(EOSResult));
			return MakeFulfilledPromise<EOS_EResult>(EOSResult).GetFuture();
		}
	}

	// Add attributes.
	for (const TPair<FSchemaAttributeId, FSchemaServiceAttributeData>& UpdatedAttribute : ServiceChanges.UpdatedAttributes)
	{
		const FLobbyAttributeTranslator<ELobbyTranslationType::ToService> AttributeTranslator(UpdatedAttribute.Value);

		EOS_LobbyModification_AddAttributeOptions AddAttributeOptions = {};
		AddAttributeOptions.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_ADDATTRIBUTE_API_LATEST, 2);
		AddAttributeOptions.Attribute = &AttributeTranslator.GetAttributeData();
		AddAttributeOptions.Visibility = AttributeTranslator.GetAttributeVisibility();

		EOSResult = EOS_LobbyModification_AddAttribute(LobbyModificationHandle, &AddAttributeOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges] EOS_LobbyModification_AddAttribute Failed: User[%ls], Lobby[%ls], Key[%ls], Value[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *UpdatedAttribute.Key.ToString().ToLower(), *ToLogString(UpdatedAttribute.Value.Value), *LexToString(EOSResult));
			return MakeFulfilledPromise<EOS_EResult>(EOSResult).GetFuture();
		}
	}

	// Remove attributes.
	for (const FSchemaAttributeId& RemovedAttribute : ServiceChanges.RemovedAttributes)
	{
		const FTCHARToUTF8 KeyConverter(*RemovedAttribute.ToString());

		EOS_LobbyModification_RemoveAttributeOptions RemoveAttributeOptions = {};
		RemoveAttributeOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_REMOVEATTRIBUTE_API_LATEST, 1);
		RemoveAttributeOptions.Key = KeyConverter.Get();
		
		EOSResult = EOS_LobbyModification_RemoveAttribute(LobbyModificationHandle, &RemoveAttributeOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges] EOS_LobbyModification_RemoveAttribute Failed: User[%ls], Lobby[%ls], Key[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *RemovedAttribute.ToString().ToLower(), *LexToString(EOSResult));
			return MakeFulfilledPromise<EOS_EResult>(EOSResult).GetFuture();
		}
	}

	// Add member attributes.
	for (const TPair<FSchemaAttributeId, FSchemaServiceAttributeData>& UpdatedAttribute : ServiceChanges.UpdatedMemberAttributes)
	{
		const FLobbyAttributeTranslator<ELobbyTranslationType::ToService> AttributeTranslator(UpdatedAttribute.Value);

		EOS_LobbyModification_AddMemberAttributeOptions AddMemberAttributeOptions = {};
		AddMemberAttributeOptions.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_ADDMEMBERATTRIBUTE_API_LATEST, 2);
		AddMemberAttributeOptions.Attribute = &AttributeTranslator.GetAttributeData();
		AddMemberAttributeOptions.Visibility = AttributeTranslator.GetAttributeVisibility();

		EOSResult = EOS_LobbyModification_AddMemberAttribute(LobbyModificationHandle, &AddMemberAttributeOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyMemberDataUpdateFromLocalChanges] EOS_LobbyModification_AddMemberAttribute Failed: User[%ls], Lobby[%ls], Key[%ls], Value[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *UpdatedAttribute.Key.ToString().ToLower(), *ToLogString(UpdatedAttribute.Value.Value), *LexToString(EOSResult));
			return MakeFulfilledPromise<EOS_EResult>(EOSResult).GetFuture();
		}
	}

	// Remove member attributes.
	for (const FSchemaAttributeId& RemovedAttribute : ServiceChanges.RemovedMemberAttributes)
	{
		const FTCHARToUTF8 KeyConverter(*RemovedAttribute.ToString());

		EOS_LobbyModification_RemoveMemberAttributeOptions RemoveMemberAttributeOptions = {};
		RemoveMemberAttributeOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYMODIFICATION_REMOVEMEMBERATTRIBUTE_API_LATEST, 1);
		RemoveMemberAttributeOptions.Key = KeyConverter.Get();

		EOSResult = EOS_LobbyModification_RemoveMemberAttribute(LobbyModificationHandle, &RemoveMemberAttributeOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyMemberDataUpdateFromLocalChanges] EOS_LobbyModification_RemoveMemberAttribute Failed: User[%ls], Lobby[%ls], Key[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(GetInfo()->GetLobbyIdEOS()), *RemovedAttribute.ToString().ToLower(), *LexToString(EOSResult));
			return MakeFulfilledPromise<EOS_EResult>(EOSResult).GetFuture();
		}
	}

	TPromise<EOS_EResult> Promise;
	TFuture<EOS_EResult> Future = Promise.GetFuture();

	// Apply lobby updates.
	EOS_Lobby_UpdateLobbyOptions UpdateLobbyOptions = {};
	UpdateLobbyOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_UPDATELOBBY_API_LATEST, 1);
	UpdateLobbyOptions.LobbyModificationHandle = LobbyModificationHandle;

	EOS_Async(EOS_Lobby_UpdateLobby, Prerequisites->LobbyInterfaceHandle, UpdateLobbyOptions,
	[LocalAccountId, LobbyInfo = GetInfo(), Promise = MoveTemp(Promise)](const EOS_Lobby_UpdateLobbyCallbackInfo* Data) mutable
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success && Data->ResultCode != EOS_EResult::EOS_NoChange)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges] EOS_Lobby_UpdateLobby Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyInfo->GetLobbyIdEOS()), *LexToString(Data->ResultCode));
		}
		else
		{
			UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDetailsEOS::ApplyLobbyDataUpdateFromLocalChanges] Succeeded: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyInfo->GetLobbyIdEOS()), *LexToString(Data->ResultCode));
		}

		Promise.EmplaceValue(Data->ResultCode);
	});
	return Future;
}

FLobbyDetailsEOS::FLobbyDetailsEOS(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	const TSharedRef<FLobbyDetailsInfoEOS>& LobbyDetailsInfo,
	FAccountId LocalAccountId,
	ELobbyDetailsSource LobbyDetailsSource,
	EOS_HLobbyDetails LobbyDetailsHandle)
	: Prerequisites(Prerequisites)
	, LobbyDetailsInfo(LobbyDetailsInfo)
	, AssociatedLocalUser(LocalAccountId)
	, LobbyDetailsSource(LobbyDetailsSource)
	, LobbyDetailsHandle(LobbyDetailsHandle)
{
}

TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>> FLobbyDetailsInfoEOS::Create(EOS_HLobbyDetails LobbyDetailsHandle)
{
	EOS_LobbyDetails_CopyInfoOptions CopyInfoOptions = {};
	CopyInfoOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYDETAILS_COPYINFO_API_LATEST, 1);

	EOS_LobbyDetails_Info* LobbyDetailsInfo = nullptr;
	EOS_EResult EOSResult = EOS_LobbyDetails_CopyInfo(LobbyDetailsHandle, &CopyInfoOptions, &LobbyDetailsInfo);
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsInfoEOS] EOS_LobbyDetails_CopyInfo Failed: Result[%ls]",
			*LexToString(EOSResult));
		return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>>(Errors::FromEOSResult(EOSResult));
	}

	return TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>>(MakeShared<FLobbyDetailsInfoEOS>(FLobbyDetailsInfoPtr(LobbyDetailsInfo)));
}

FLobbyDetailsInfoEOS::FLobbyDetailsInfoEOS(FLobbyDetailsInfoPtr&& InLobbyDetailsInfo)
	: LobbyDetailsInfo(MoveTemp(InLobbyDetailsInfo))
{
	const FLobbyBucketIdTranslator<ELobbyTranslationType::FromService> BucketTranslator(LobbyDetailsInfo->BucketId);
	BucketId = BucketTranslator.GetBucketId();

	if (!BucketId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyDetailsInfoEOS] Failed: Unable to parse lobby bucket id. Lobby[%ls], Bucket[%ls]",
			UTF8_TO_TCHAR(LobbyDetailsInfo->LobbyId), UTF8_TO_TCHAR(LobbyDetailsInfo->BucketId));
	}
}

FLobbyDataEOS::~FLobbyDataEOS()
{
	if (UnregisterFn)
	{
		UnregisterFn(LobbyClientData->GetPublicData().LobbyId);
	}
}

void FLobbyDataEOS::AddUserLobbyDetails(FAccountId LocalAccountId, const TSharedPtr<FLobbyDetailsEOS>& LobbyDetails)
{
	if (TSharedPtr<FLobbyDetailsEOS> ExistingDetails = GetUserLobbyDetails(LocalAccountId))
	{
		if (ExistingDetails->GetDetailsSource() < LobbyDetails->GetDetailsSource())
		{
			return;
		}
	}

	UserLobbyDetails.Add(LocalAccountId, LobbyDetails);
}

TSharedPtr<FLobbyDetailsEOS> FLobbyDataEOS::GetUserLobbyDetails(FAccountId LocalAccountId) const
{
	const TSharedPtr<FLobbyDetailsEOS>* Result = UserLobbyDetails.Find(LocalAccountId);
	return Result ? *Result : TSharedPtr<FLobbyDetailsEOS>();
}

TSharedPtr<FLobbyDetailsEOS> FLobbyDataEOS::GetActiveLobbyDetails() const
{
	TSharedPtr<FLobbyDetailsEOS> FoundDetails;

	for (const TPair<FAccountId, TSharedPtr<FLobbyDetailsEOS>>& LobbyDetails : UserLobbyDetails)
	{
		if (LobbyDetails.Value->GetDetailsSource() == ELobbyDetailsSource::Active)
		{
			FoundDetails = LobbyDetails.Value;
			break;
		}
	}

	return FoundDetails;
}

FLobbyDataEOS::FLobbyDataEOS(
	const TSharedRef<FLobbyClientData>& LobbyClientData,
	const TSharedRef<FLobbyDetailsInfoEOS>& LobbyDetailsInfo,
	FUnregisterFn UnregisterFn)
	: LobbyClientData(LobbyClientData)
	, LobbyDetailsInfo(LobbyDetailsInfo)
	, UnregisterFn(MoveTemp(UnregisterFn))
	, LobbyIdString(UTF8_TO_TCHAR(LobbyDetailsInfo->GetLobbyIdEOS()))
{
}

TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> FLobbyDataEOS::Create(
	TSharedRef<FLobbyPrerequisitesEOS> Prerequisites,
	FLobbyId LobbyId,
	const TSharedRef<FLobbyDetailsEOS>& LobbyDetails,
	FUnregisterFn UnregisterFn)
{
	TPromise<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> Promise;
	TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> Future = Promise.GetFuture();
	
	LobbyDetails->GetLobbySnapshot(false)
	.Next(
	[
		Promise = MoveTemp(Promise),
		LobbyId,
		LobbyDetails,
		Prerequisites,
		UnregisterFn = MoveTemp(UnregisterFn)
	]
	(TDefaultErrorResultInternal<FLobbyServiceSnapshot>&& Result) mutable
	{
		if (Result.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDataEOS::Create] FLobbyDetailsEOS::GetLobbySnapshot Failed: Lobby[%ls], Result[%ls]",
				*ToLogString(LobbyId), *Result.GetErrorValue().GetLogString());
			Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
			return;
		}

		FLobbyServiceSnapshot LobbySnapshot = MoveTemp(Result.GetOkValue());
		TSharedRef<FLobbyClientData> NewLobbyClientData = MakeShared<FLobbyClientData>(LobbyId, Prerequisites->SchemaRegistry);

		// Fetch member data and apply them to the lobby.
		TMap<FAccountId, FLobbyMemberServiceSnapshot> MemberSnapshots;
		for (FAccountId MemberAccountId : LobbySnapshot.Members)
		{
			TDefaultErrorResultInternal<FLobbyMemberServiceSnapshot> LobbyMemberSnapshotResult = LobbyDetails->GetLobbyMemberSnapshot(MemberAccountId);
			if (LobbyMemberSnapshotResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyDataEOS::Create] FLobbyDetailsEOS::GetLobbyMemberSnapshot Failed: Lobby[%ls], User[%ls], Result[%ls]",
					*ToLogString(LobbyId), *ToLogString(MemberAccountId), *Result.GetErrorValue().GetLogString());
				Promise.EmplaceValue(MoveTemp(LobbyMemberSnapshotResult.GetErrorValue()));
				return;
			}

			MemberSnapshots.Emplace(MemberAccountId, MoveTemp(LobbyMemberSnapshotResult.GetOkValue()));
		}

		TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareServiceLobbySnapshotResult =
			NewLobbyClientData->PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(MemberSnapshots), {} });

		if (PrepareServiceLobbySnapshotResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDataEOS::Create] FLobbyClientData::PrepareServiceSnapshot Failed: Lobby[%ls], Result[%ls]",
				*ToLogString(LobbyId), *PrepareServiceLobbySnapshotResult.GetErrorValue().GetLogString());
			Promise.EmplaceValue(MoveTemp(PrepareServiceLobbySnapshotResult.GetErrorValue()));
			return;
		}

		NewLobbyClientData->CommitServiceSnapshot({});

		UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDataEOS::Create] Succeeded: Lobby[%ls]",
			UTF8_TO_TCHAR(LobbyDetails->GetInfo()->GetLobbyIdEOS()));
		Promise.EmplaceValue(MakeShared<FLobbyDataEOS>(
			NewLobbyClientData,
			LobbyDetails->GetInfo(),
			MoveTemp(UnregisterFn)));
	});

	return Future;
}

FLobbyDataRegistryEOS::FLobbyDataRegistryEOS(const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites)
	: Prerequisites(Prerequisites)
{
}

TSharedPtr<FLobbyDataEOS> FLobbyDataRegistryEOS::Find(EOS_LobbyId LobbyIdEOS) const
{
	const TWeakPtr<FLobbyDataEOS>* Result = LobbyIdIndex.Find(UTF8_TO_TCHAR(LobbyIdEOS));
	return Result ? Result->Pin() : TSharedPtr<FLobbyDataEOS>();
}

TSharedPtr<FLobbyDataEOS> FLobbyDataRegistryEOS::Find(FLobbyId LobbyId) const
{
	const TWeakPtr<FLobbyDataEOS>* Result = LobbyIdHandleIndex.Find(LobbyId);
	return Result ? Result->Pin() : TSharedPtr<FLobbyDataEOS>();
}

TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails(FAccountId LocalAccountId, const TSharedRef<FLobbyDetailsEOS>& LobbyDetails)
{
	// Check if lobby data already exists.
	if (TSharedPtr<FLobbyDataEOS> FindResult = Find(LobbyDetails->GetInfo()->GetLobbyIdEOS()))
	{
		UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails] Succeeded: Lobby data already exists. User[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *FindResult->GetLobbyIdString());

		FindResult->AddUserLobbyDetails(LocalAccountId, LobbyDetails);
		return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>>(FindResult.ToSharedRef()).GetFuture();
	}

	TPromise<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> Promise;
	TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> Future = Promise.GetFuture();

	// Create new lobby id object.
	const FLobbyId LobbyId = FLobbyId(EOnlineServices::Epic, NextHandleIndex++);

	// Create lobby data from lobby details.
	FLobbyDataEOS::Create(Prerequisites, LobbyId, LobbyDetails, MakeUnregisterFn())
	.Next([WeakThis = AsWeak(), Promise = MoveTemp(Promise), LocalAccountId, LobbyDetails](TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>&& Result) mutable
	{
		TSharedPtr<FLobbyDataRegistryEOS> StrongThis = WeakThis.Pin();
		if (!StrongThis.IsValid())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails] Failed: FLobbyDataRegistryEOS has been destroyed. User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyDetails->GetInfo()->GetLobbyIdEOS()), *Result.GetErrorValue().GetLogString());
			Promise.EmplaceValue(Errors::NotFound());
			return;
		}

		if (Result.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails] FLobbyDataEOS::Create Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(LobbyDetails->GetInfo()->GetLobbyIdEOS()), *Result.GetErrorValue().GetLogString());
			Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
			return;
		}

		UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails] Succeeded: Created new lobby data. User[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *Result.GetOkValue()->GetLobbyIdString());

		StrongThis->Register(Result.GetOkValue());
		Result.GetOkValue()->AddUserLobbyDetails(LocalAccountId, LobbyDetails);
		Promise.EmplaceValue(MoveTemp(Result.GetOkValue()));
	});

	return Future;
}

void FLobbyDataRegistryEOS::Register(const TSharedRef<FLobbyDataEOS>& LobbyIdHandleData)
{
	LobbyIdIndex.Add(LobbyIdHandleData->GetLobbyIdEOS(), LobbyIdHandleData);
	LobbyIdHandleIndex.Add(LobbyIdHandleData->GetLobbyIdHandle(), LobbyIdHandleData);
}

void FLobbyDataRegistryEOS::Unregister(FLobbyId LobbyId)
{
	if (TSharedPtr<FLobbyDataEOS> HandleData = Find(LobbyId))
	{
		LobbyIdIndex.Remove(HandleData->GetLobbyIdEOS());
		LobbyIdHandleIndex.Remove(HandleData->GetLobbyIdHandle());
	}
}

FLobbyDataEOS::FUnregisterFn FLobbyDataRegistryEOS::MakeUnregisterFn()
{
	return [WeakThis = AsWeak()](FLobbyId LobbyId)
	{
		if (TSharedPtr<FLobbyDataRegistryEOS> StrongThis = WeakThis.Pin())
		{
			StrongThis->Unregister(LobbyId);
		}
	};
}

TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>> FLobbyInviteDataEOS::CreateFromInviteId(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	const TSharedRef<FLobbyDataRegistryEOS>& LobbyDataRegistry,
	FAccountId LocalAccountId,
	const char* InviteIdEOS,
	EOS_ProductUserId Sender)
{
	TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> LobbyDetailsResult = FLobbyDetailsEOS::CreateFromInviteId(Prerequisites, LocalAccountId, InviteIdEOS);
	if (LobbyDetailsResult.IsError())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyInviteDataEOS::CreateFromInviteId] FLobbyDetailsEOS::CreateFromInviteId Failed: User[%ls], InviteId[%ls], Result[%ls]",
			*ToLogString(LocalAccountId), UTF8_TO_TCHAR(InviteIdEOS), *LobbyDetailsResult.GetErrorValue().GetLogString());

		return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>>(MoveTemp(LobbyDetailsResult.GetErrorValue())).GetFuture();
	}

	TSharedRef<FLobbyDetailsEOS> LobbyDetails = LobbyDetailsResult.GetOkValue();
	TPromise<TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>> Promise;
	TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>> Future = Promise.GetFuture();

	// Search for existing lobby data so that the LobbyId will match.
	TSharedRef<FLobbyInviteIdEOS> InviteId = MakeShared<FLobbyInviteIdEOS>(InviteIdEOS);
	LobbyDataRegistry->FindOrCreateFromLobbyDetails(LocalAccountId, LobbyDetails)
	.Next([Promise = MoveTemp(Promise), InviteId, LocalAccountId, Sender, LobbyDetails](TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>&& Result) mutable
	{
		if (Result.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyInviteDataEOS::CreateFromInviteId] FLobbyDataEOS::FindOrCreateFromLobbyDetails Failed: User[%ls], InviteId[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(InviteId->Get()), UTF8_TO_TCHAR(LobbyDetails->GetInfo()->GetLobbyIdEOS()), *Result.GetErrorValue().GetLogString());

			Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
			return;
		}

		// Once the lobby data has been resolved the FAccountId for the sender is expected to be in the AccountID cache.
		const FAccountId SenderAccountId = FindAccountId(LocalAccountId.GetOnlineServicesType(), Sender);
		if (!SenderAccountId.IsValid())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyInviteDataEOS::CreateFromInviteId] Failed: Unable to find account id for sender. User[%ls], InviteId[%ls], Lobby[%ls]",
				*ToLogString(LocalAccountId), UTF8_TO_TCHAR(InviteId->Get()), UTF8_TO_TCHAR(LobbyDetails->GetInfo()->GetLobbyIdEOS()));

			Promise.EmplaceValue(Errors::InvalidState());
			return;
		}

		UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbyInviteDataEOS::CreateFromInviteId] Succeeded: User[%ls], Sender[%ls], InviteId[%ls], Lobby[%ls]",
			*ToLogString(LocalAccountId), *ToLogString(SenderAccountId), UTF8_TO_TCHAR(InviteId->Get()), *Result.GetOkValue()->GetLobbyIdString());
		Promise.EmplaceValue(MakeShared<FLobbyInviteDataEOS>(InviteId, LocalAccountId, SenderAccountId, LobbyDetails, Result.GetOkValue()));
	});

	return Future;
}

FLobbyInviteDataEOS::FLobbyInviteDataEOS(
	const TSharedRef<FLobbyInviteIdEOS>& InviteIdEOS,
	FAccountId Receiver,
	FAccountId Sender,
	const TSharedRef<FLobbyDetailsEOS>& LobbyDetails,
	const TSharedRef<FLobbyDataEOS>& LobbyData)
	: InviteIdEOS(InviteIdEOS)
	, Receiver(Receiver)
	, Sender(Sender)
	, LobbyDetails(LobbyDetails)
	, LobbyData(LobbyData)
	, InviteId(UTF8_TO_TCHAR(InviteIdEOS->Get()))
{
}

TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>> FLobbySearchEOS::Create(
	const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
	const TSharedRef<FLobbyDataRegistryEOS>& LobbyRegistry,
	const FLobbySearchParameters& Params)
{
	TSharedRef<FSearchHandle> SearchHandle = MakeShared<FSearchHandle>();

	EOS_Lobby_CreateLobbySearchOptions CreateLobbySearchOptions = {};
	CreateLobbySearchOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_CREATELOBBYSEARCH_API_LATEST, 1);
	CreateLobbySearchOptions.MaxResults = Params.MaxResults;

	EOS_EResult EOSResult = EOS_Lobby_CreateLobbySearch(Prerequisites->LobbyInterfaceHandle, &CreateLobbySearchOptions, &SearchHandle->Get());
	if (EOSResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] EOS_Lobby_CreateLobbySearch Failed: User[%ls], Result[%ls]",
			*ToLogString(Params.LocalAccountId), *LexToString(EOSResult));
		return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::FromEOSResult(EOSResult)).GetFuture();
	}

	if (Params.LobbyId) // Search for specific lobby.
	{
		TSharedPtr<FLobbyDataEOS> LobbyData = LobbyRegistry->Find(*Params.LobbyId);
		if (!LobbyData)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] Failed: Unable to find lobby data for lobby. User[%ls], Lobby[%ls]",
				*ToLogString(Params.LocalAccountId), *ToLogString(Params.LobbyId));
			return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::InvalidParams()).GetFuture();
		}

		EOS_LobbySearch_SetLobbyIdOptions SetLobbyIdOptions = {};
		SetLobbyIdOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETLOBBYID_API_LATEST, 1);
		SetLobbyIdOptions.LobbyId = LobbyData->GetLobbyIdEOS();

		EOSResult = EOS_LobbySearch_SetLobbyId(SearchHandle->Get(), &SetLobbyIdOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] EOS_LobbySearch_SetLobbyId Failed: User[%ls], Lobby[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *LobbyData->GetLobbyIdString(), *LexToString(EOSResult));
			return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::FromEOSResult(EOSResult)).GetFuture();
		}
	}
	else if (Params.TargetUser) // Search for specific user.
	{
		EOS_LobbySearch_SetTargetUserIdOptions SetTargetUserIdOptions = {};
		SetTargetUserIdOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETTARGETUSERID_API_LATEST, 1);
		SetTargetUserIdOptions.TargetUserId = GetProductUserIdChecked(*Params.TargetUser);

		EOSResult = EOS_LobbySearch_SetTargetUserId(SearchHandle->Get(), &SetTargetUserIdOptions);
		if (EOSResult != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] EOS_LobbySearch_SetTargetUserId Failed: User[%ls], Target[%ls], Result[%ls]",
				*ToLogString(Params.LocalAccountId), *ToLogString(*Params.TargetUser), *LexToString(EOSResult));
			return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::FromEOSResult(EOSResult)).GetFuture();
		}
	}
	else // Search using parameters.
	{
		// Bucket id.
		{
			const FLobbyBucketIdTranslator<ELobbyTranslationType::ToService> BucketTranslator(Prerequisites->BucketId);

			EOS_Lobby_AttributeData AttributeData;
			AttributeData.ApiVersion = 1;
			AttributeData.Key = EOS_LOBBY_SEARCH_BUCKET_ID;
			AttributeData.ValueType = EOS_ELobbyAttributeType::EOS_AT_STRING;
			AttributeData.Value.AsUtf8 = BucketTranslator.GetBucketIdEOS();
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ATTRIBUTEDATA_API_LATEST, 1);

			EOS_LobbySearch_SetParameterOptions SetParameterOptions = {};
			SetParameterOptions.ApiVersion = 1;
			SetParameterOptions.Parameter = &AttributeData;
			SetParameterOptions.ComparisonOp = EOS_EComparisonOp::EOS_CO_EQUAL;
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETPARAMETER_API_LATEST, 1);

			EOSResult = EOS_LobbySearch_SetParameter(SearchHandle->Get(), &SetParameterOptions);
			if (EOSResult != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] EOS_LobbySearch_SetParameter Failed: User[%ls], Key[%ls], Result[%ls]",
					*ToLogString(Params.LocalAccountId), UTF8_TO_TCHAR(EOS_LOBBY_SEARCH_BUCKET_ID), *LexToString(EOSResult));
				return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::FromEOSResult(EOSResult)).GetFuture();
			}
		}

		// Create schema category instance to verify search filter attributes using the base lobby schema.
		FSchemaCategoryInstance SchemaCategoryInstance(FSchemaId(), LobbyBaseSchemaId, LobbySchemaCategoryId, Prerequisites->SchemaRegistry);
		for (const FFindLobbySearchFilter& Filter : Params.Filters)
		{
			FSchemaServiceAttributeId SchemaServiceAttributeId;
			ESchemaServiceAttributeFlags SchemaServiceAttributeFlags = ESchemaServiceAttributeFlags::None;
			if (!SchemaCategoryInstance.VerifyBaseAttributeData(Filter.AttributeName, Filter.ComparisonValue, SchemaServiceAttributeId, SchemaServiceAttributeFlags))
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] VerifyBaseAttributeData failed: User[%ls], Key[%ls], Value[%ls]",
					*ToLogString(Params.LocalAccountId), *Filter.AttributeName.ToString().ToLower(), *ToLogString(Filter.ComparisonValue));
				return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::InvalidParams()).GetFuture();
			}

			const FLobbyAttributeTranslator<ELobbyTranslationType::ToService> AttributeTranslator(
				{ SchemaServiceAttributeId, SchemaServiceAttributeFlags, Filter.ComparisonValue });

			EOS_LobbySearch_SetParameterOptions SetParameterOptions = {};
			SetParameterOptions.ApiVersion = 1;
			SetParameterOptions.Parameter = &AttributeTranslator.GetAttributeData();
			SetParameterOptions.ComparisonOp = Private::TranslateSearchComparison(Filter.ComparisonOp);
			UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_SETPARAMETER_API_LATEST, 1);

			EOSResult = EOS_LobbySearch_SetParameter(SearchHandle->Get(), &SetParameterOptions);
			if (EOSResult != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] EOS_LobbySearch_SetParameter Failed: User[%ls], Key[%ls], Value[%ls], Result[%ls]",
					*ToLogString(Params.LocalAccountId), *Filter.AttributeName.ToString().ToLower(), *ToLogString(Filter.ComparisonValue), *LexToString(EOSResult));
				return MakeFulfilledPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>>(Errors::FromEOSResult(EOSResult)).GetFuture();
			}
		}
	}

	TPromise<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>> Promise;
	TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>> Future = Promise.GetFuture();

	EOS_LobbySearch_FindOptions FindOptions = {};
	FindOptions.ApiVersion = 1;
	FindOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_FIND_API_LATEST, 1);

	EOS_Async(EOS_LobbySearch_Find, SearchHandle->Get(), FindOptions,
	[
		Promise = MoveTemp(Promise),
		Prerequisites,
		LobbyRegistry,
		LocalAccountId = Params.LocalAccountId,
		SearchHandle
	](const EOS_LobbySearch_FindCallbackInfo* Data) mutable
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] EOS_LobbySearch_Find Failed: User[%ls], Result[%ls]",
				*ToLogString(LocalAccountId), *LexToString(Data->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Data->ResultCode));
			return;
		}

		TArray<TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>>> ResolvedLobbyDetails;

		EOS_LobbySearch_GetSearchResultCountOptions GetSearchResultCountOptions = {};
		GetSearchResultCountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBYSEARCH_GETSEARCHRESULTCOUNT_API_LATEST, 1);
		
		const uint32_t NumSearchResults = EOS_LobbySearch_GetSearchResultCount(SearchHandle->Get(), &GetSearchResultCountOptions);

		for (uint32_t SearchResultIndex = 0; SearchResultIndex < NumSearchResults; ++SearchResultIndex)
		{
			TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> Result = FLobbyDetailsEOS::CreateFromSearchResult(
				Prerequisites, LocalAccountId, SearchHandle->Get(), SearchResultIndex);
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] FLobbyDetailsEOS::CreateFromSearchResult Failed: User[%ls], Result[%ls]",
					*ToLogString(LocalAccountId), *Result.GetErrorValue().GetLogString());
				Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
				return;
			}

			TPromise<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> ResolveLobbyDetailsPromise;
			ResolvedLobbyDetails.Add(ResolveLobbyDetailsPromise.GetFuture());

			LobbyRegistry->FindOrCreateFromLobbyDetails(LocalAccountId, Result.GetOkValue())
			.Next([ResolveLobbyDetailsPromise = MoveTemp(ResolveLobbyDetailsPromise)](TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>&& Result) mutable
			{
				ResolveLobbyDetailsPromise.EmplaceValue(MoveTemp(Result));
			});
		}

		WhenAll(MoveTemp(ResolvedLobbyDetails))
		.Next([LocalAccountId, Promise = MoveTemp(Promise), SearchHandle](TArray<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>>&& ResultArray) mutable
		{
			TArray<TSharedRef<FLobbyDataEOS>> ResolvedResults;
			ResolvedResults.Reserve(ResultArray.Num());

			for (TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>& Result : ResultArray)
			{
				if (Result.IsError())
				{
					UE_LOGF(LogOnlineServices, Warning, "[FLobbySearchEOS::Create] FLobbyDataRegistryEOS::FindOrCreateFromLobbyDetails Failed: Unable to resolve search result. User[%ls], Result[%ls]",
						*ToLogString(LocalAccountId), *Result.GetErrorValue().GetLogString());
					Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
					return;
				}

				ResolvedResults.Add(MoveTemp(Result.GetOkValue()));
			}

			UE_LOGF(LogOnlineServices, VeryVerbose, "[FLobbySearchEOS::Create] Succeeded: User[%ls], NumResults[%d]",
				*ToLogString(LocalAccountId), ResolvedResults.Num());
			Promise.EmplaceValue(MakeShared<FLobbySearchEOS>(SearchHandle, MoveTemp(ResolvedResults)));
		});
	});

	return Future;
}

TArray<TSharedRef<const FLobby>> FLobbySearchEOS::GetLobbyResults() const
{
	TArray<TSharedRef<const FLobby>> Result;
	Result.Reserve(Lobbies.Num());

	for (const TSharedRef<FLobbyDataEOS>& LobbyData : Lobbies)
	{
		Result.Add(LobbyData->GetLobbyClientData()->GetPublicDataPtr());
	}

	return Result;
}

const TArray<TSharedRef<FLobbyDataEOS>>& FLobbySearchEOS::GetLobbyData()
{
	return Lobbies;
}

FLobbySearchEOS::FLobbySearchEOS(const TSharedRef<FSearchHandle>& SearchHandle, TArray<TSharedRef<FLobbyDataEOS>>&& Lobbies)
	: SearchHandle(SearchHandle)
	, Lobbies(Lobbies)
{
}

FString ToLogString(const FLobbyDataEOS& LobbyData)
{
	return FString::Printf(TEXT("[%s:%s]"), *ToLogString(LobbyData.GetLobbyIdHandle()), *LobbyData.GetLobbyIdString());
}

/* UE::Online */ }
