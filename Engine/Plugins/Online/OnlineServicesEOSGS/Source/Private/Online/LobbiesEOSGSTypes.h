// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/LobbiesCommonTypes.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include <eos_common.h>
#include <eos_lobby_types.h>
#include <eos_ui_types.h>

enum class ESPMode : uint8;
namespace UE::Online { template <typename Result> class TDefaultErrorResultInternal; }
template <typename ResultType> class TFuture;


namespace UE::Online {

class FLobbySchemaRegistry;
class FLobbyDataRegistryEOS;
class FLobbyDetailsInfoEOS;
class FLobbyInviteDataEOSRegistry;
class FLobbySchema;
struct FLobbyEvents;
struct FLobbyMemberServiceSnapshot;
class IEpicProductUserIdResolver;

//--------------------------------------------------------------------------------------------------
// Translated types
//--------------------------------------------------------------------------------------------------

#define UE_ONLINE_LOBBIES_BUCKET_ID_EOS_SEPARATOR TEXT("|")

class FLobbyBucketIdEOS
{
public:
	FLobbyBucketIdEOS() = default;
	FLobbyBucketIdEOS(const FString& ProductName, int32 ProductVersion);

	FLobbyBucketIdEOS(const FLobbyBucketIdEOS&) = default;
	FLobbyBucketIdEOS(FLobbyBucketIdEOS&&) = default;
	FLobbyBucketIdEOS& operator=(const FLobbyBucketIdEOS&) = default;
	FLobbyBucketIdEOS& operator=(FLobbyBucketIdEOS&&) = default;

	bool operator==(const FLobbyBucketIdEOS& Other) const
	{
		return ProductName == Other.ProductName && ProductVersion == Other.ProductVersion;
	}

	bool operator!=(const FLobbyBucketIdEOS& Other) const
	{
		return !(*this == Other);
	}

	const FString& GetProductName() const { return ProductName; }
	int32 GetProductVersion() const { return ProductVersion; }
	bool IsValid() const { return !ProductName.IsEmpty(); }

private:
	FString ProductName;
	int32 ProductVersion;
};

FString ToLogString(const FLobbyBucketIdEOS& BucketId);

//--------------------------------------------------------------------------------------------------
// Translators
//--------------------------------------------------------------------------------------------------
enum class ELobbyTranslationType
{
	ToService,
	FromService
};

//--------------------------------------------------------------------------------------------------
// Lobby attribute

template <ELobbyTranslationType>
class FLobbyAttributeTranslator
{
public:
};

template <>
class FLobbyAttributeTranslator<ELobbyTranslationType::ToService>
{
public:
	FLobbyAttributeTranslator(const FSchemaServiceAttributeData& FromAttributeData);

	const EOS_Lobby_AttributeData& GetAttributeData() const { return AttributeData; }
	const EOS_ELobbyAttributeVisibility& GetAttributeVisibility() const { return AttributeVisibility; }

private:
	FTCHARToUTF8 KeyConverterStorage;
	TOptional<FTCHARToUTF8> ValueConverterStorage;
	EOS_Lobby_AttributeData AttributeData;
	EOS_ELobbyAttributeVisibility AttributeVisibility;
};

template <>
class FLobbyAttributeTranslator<ELobbyTranslationType::FromService>
{
public:
	FLobbyAttributeTranslator(const EOS_Lobby_AttributeData& FromAttributeData);

	const TPair<FSchemaAttributeId, FSchemaVariant>& GetAttributeData() const { return AttributeData; }
	TPair<FSchemaAttributeId, FSchemaVariant>& GetMutableAttributeData() { return AttributeData; }

private:
	TPair<FSchemaAttributeId, FSchemaVariant> AttributeData;
};

//--------------------------------------------------------------------------------------------------
// Bucket id

template <ELobbyTranslationType>
class FLobbyBucketIdTranslator;

template <>
class FLobbyBucketIdTranslator<ELobbyTranslationType::ToService>
{
public:
	FLobbyBucketIdTranslator(const FLobbyBucketIdEOS& BucketId);

	const char* GetBucketIdEOS() const { return BucketConverterStorage.Get(); }

private:
	FTCHARToUTF8 BucketConverterStorage;
};

template <>
class FLobbyBucketIdTranslator<ELobbyTranslationType::FromService>
{
public:
	FLobbyBucketIdTranslator(const char* BucketIdEOS);

	const FLobbyBucketIdEOS& GetBucketId() const { return BucketId; }
	FLobbyBucketIdEOS& GetMutableBucketId() { return BucketId; }

private:
	FLobbyBucketIdEOS BucketId;
};

//--------------------------------------------------------------------------------------------------
// Enumerations

inline EOS_ELobbyPermissionLevel TranslateJoinPolicy(ELobbyJoinPolicy JoinPolicy)
{
	switch (JoinPolicy)
	{
	case ELobbyJoinPolicy::PublicAdvertised:	return EOS_ELobbyPermissionLevel::EOS_LPL_PUBLICADVERTISED;
	case ELobbyJoinPolicy::PublicNotAdvertised:	return EOS_ELobbyPermissionLevel::EOS_LPL_JOINVIAPRESENCE;
	default:									checkNoEntry(); // Intentional fallthrough
	case ELobbyJoinPolicy::InvitationOnly:		return EOS_ELobbyPermissionLevel::EOS_LPL_INVITEONLY;
	};
}

inline ELobbyJoinPolicy TranslateJoinPolicy(EOS_ELobbyPermissionLevel JoinPolicy)
{
	switch (JoinPolicy)
	{
	case EOS_ELobbyPermissionLevel::EOS_LPL_PUBLICADVERTISED:	return ELobbyJoinPolicy::PublicAdvertised;
	case EOS_ELobbyPermissionLevel::EOS_LPL_JOINVIAPRESENCE:	return ELobbyJoinPolicy::PublicNotAdvertised;
	default:									checkNoEntry(); // Intentional fallthrough
	case EOS_ELobbyPermissionLevel::EOS_LPL_INVITEONLY:	return ELobbyJoinPolicy::InvitationOnly;
	};
}

//--------------------------------------------------------------------------------------------------
// Structures.
//--------------------------------------------------------------------------------------------------

using FLobbySearchParameters = FFindLobbies::Params;

/**
* Common components required to handle lobby requests.
* To handle lifetime issues, only FLobbiesEOS should contain a strong reference to prerequisites.
* Any other component should only store a weak reference.
*/
struct FLobbyPrerequisitesEOS
{
	EOS_HLobby LobbyInterfaceHandle = {};
	IEpicProductUserIdResolver* ResolverInterface;
	FLobbyBucketIdEOS BucketId;
	TSharedRef<FSchemaRegistry> SchemaRegistry;
};

/**
 * Lobby details are created for each user within the EOS lobby client. Certain operations such as
 * joining a lobby or applying an update to a lobby / member attributes require using the correct
 * lobby details handle.
 */
enum class ELobbyDetailsSource
{
	/**
	 * The lobby has been joined and is considered active.
	 * Valid for lobby / member updates.
	 */
	Active,

	/**
	 * The details originated from an invitation sent to the user.
	 * Valid for joining.
	 */
	Invite,

	/**
	 * The details originated from an event generated by the EOS overlay.
	 * Valid for joining.
	 */
	UiEvent,

	/**
	 * The details originated from a lobby search result.
	 * Valid for joining.
	 */
	Search
};

/** Lobby details is created based on the passed in user and is required to join a lobby. */
class FLobbyDetailsEOS final : public TSharedFromThis<FLobbyDetailsEOS>
{
public:
	UE_NONCOPYABLE(FLobbyDetailsEOS);

	static TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> CreateFromLobbyId(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		FAccountId LocalAccountId,
		EOS_LobbyId LobbyIdEOS);
	static TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> CreateFromLobbyId(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		FAccountId LocalAccountId,
		const FString& LobbyId);
	static TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> CreateFromInviteId(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		FAccountId LocalAccountId,
		const char* InviteId);
	static TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> CreateFromUiEventId(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		FAccountId LocalAccountId,
		EOS_UI_EventId UiEventId);
	static TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsEOS>> CreateFromSearchResult(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		FAccountId LocalAccountId,
		EOS_HLobbySearch SearchHandle,
		uint32_t ResultIndex);

	FLobbyDetailsEOS() = delete;
	~FLobbyDetailsEOS();

	EOS_HLobbyDetails GetEOSHandle() const { return LobbyDetailsHandle; }
	const TSharedRef<FLobbyDetailsInfoEOS>& GetInfo() const { return LobbyDetailsInfo; }
	ELobbyDetailsSource GetDetailsSource() const { return LobbyDetailsSource; }
	FAccountId GetAssociatedUser() const { return AssociatedLocalUser; }

	/**
	 * Check whether this lobbies bucket is compatible with the currently running application.
	 */
	bool IsBucketCompatible() const;

	/**
	 * Retrieve a lobby data snapshot from the EOS lobby details object.
	 * The lobby schema will be used to translate attribute data before returning.
	 */
	TFuture<TDefaultErrorResultInternal<FLobbyServiceSnapshot>> GetLobbySnapshot(bool bCopyMemberData) const;

	/**
	 * Retrieve lobby member data snapshot from the EOS lobby details object.
	 * The lobby schema will be used to translate attribute data before returning.
	 */
	TDefaultErrorResultInternal<FLobbyMemberServiceSnapshot> GetLobbyMemberSnapshot(FAccountId MemberAccountId) const;

	/**
	 * Apply client side lobby changes to the lobby service.
	 */
	TFuture<EOS_EResult> ApplyLobbyDataUpdateFromLocalChanges(FAccountId LocalAccountId, const FLobbyClientServiceChanges& ServiceChanges) const;

private:
	template <typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
	FLobbyDetailsEOS(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		const TSharedRef<FLobbyDetailsInfoEOS>& LobbyDetailsInfo,
		FAccountId LocalAccountId,
		ELobbyDetailsSource LobbyDetailsSource,
		EOS_HLobbyDetails LobbyDetailsHandle);

	static const EOS_HLobbyDetails InvalidLobbyDetailsHandle;
	TSharedRef<FLobbyPrerequisitesEOS> Prerequisites;
	TSharedRef<FLobbyDetailsInfoEOS> LobbyDetailsInfo;
	FAccountId AssociatedLocalUser;
	ELobbyDetailsSource LobbyDetailsSource;
	EOS_HLobbyDetails LobbyDetailsHandle = InvalidLobbyDetailsHandle;
};

/** Lobby details info allows direct access to some of the lobby properties. */
class FLobbyDetailsInfoEOS final
{
public:
	UE_NONCOPYABLE(FLobbyDetailsInfoEOS);
	FLobbyDetailsInfoEOS() = delete;

	static TDefaultErrorResultInternal<TSharedRef<FLobbyDetailsInfoEOS>> Create(EOS_HLobbyDetails LobbyDetailsHandle);

	EOS_LobbyId GetLobbyIdEOS() const { return LobbyDetailsInfo->LobbyId; }
	int32 GetMaxMembers() const { return LobbyDetailsInfo->MaxMembers; }
	EOS_ELobbyPermissionLevel GetPermissionLevel() const { return LobbyDetailsInfo->PermissionLevel; }
	const FLobbyBucketIdEOS& GetBucketId() const { return BucketId; }

private:
	struct FEOSLobbyDetailsInfoDeleter
	{
		void operator()(EOS_LobbyDetails_Info* Ptr) const
		{
			EOS_LobbyDetails_Info_Release(Ptr);
		}
	};

	using FLobbyDetailsInfoPtr = TUniquePtr<EOS_LobbyDetails_Info, FEOSLobbyDetailsInfoDeleter>;

	template <typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
	FLobbyDetailsInfoEOS(FLobbyDetailsInfoPtr&& LobbyDetailsInfo);

	TUniquePtr<EOS_LobbyDetails_Info, FEOSLobbyDetailsInfoDeleter> LobbyDetailsInfo;
	FLobbyBucketIdEOS BucketId;
};

/** Lobby data is the bookkeeping object for a lobby. It contains the client-side representation of a lobby. */
class FLobbyDataEOS final : public TSharedFromThis<FLobbyDataEOS>
{
public:
	using FUnregisterFn = TFunction<void(FLobbyId)>;

	FLobbyDataEOS() = delete;

	static TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> Create(
		TSharedRef<FLobbyPrerequisitesEOS> Prerequisites,
		FLobbyId LobbyId,
		const TSharedRef<FLobbyDetailsEOS>& LobbyDetails,
		FUnregisterFn UnregisterFn = FUnregisterFn());

	~FLobbyDataEOS();

	FLobbyId GetLobbyIdHandle() const { return LobbyClientData->GetPublicData().LobbyId; }
	const TSharedRef<FLobbyClientData>& GetLobbyClientData() const { return LobbyClientData; }
	EOS_LobbyId GetLobbyIdEOS() const { return LobbyDetailsInfo->GetLobbyIdEOS(); }
	const FString& GetLobbyIdString() const { return LobbyIdString; }

	void AddUserLobbyDetails(FAccountId LocalAccountId, const TSharedPtr<FLobbyDetailsEOS>& LobbyDetails);
	TSharedPtr<FLobbyDetailsEOS> GetUserLobbyDetails(FAccountId LocalAccountId) const;

	/**
	 * Active lobby details are needed to process lobby notifications. Search for and return active
	 * lobby details if available.
	 */
	TSharedPtr<FLobbyDetailsEOS> GetActiveLobbyDetails() const;

private:
	template <typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FLobbyDataEOS(
		const TSharedRef<FLobbyClientData>& LobbyClientData,
		const TSharedRef<FLobbyDetailsInfoEOS>& LobbyDetailsInfo,
		FUnregisterFn UnregisterFn);

	TSharedRef<FLobbyClientData> LobbyClientData;
	TSharedRef<FLobbyDetailsInfoEOS> LobbyDetailsInfo;
	FUnregisterFn UnregisterFn;
	FString LobbyIdString;
	TMap<FAccountId, TSharedPtr<FLobbyDetailsEOS>> UserLobbyDetails;
};

// Todo: implement handling for FOnlineIdRegistryRegistry.
class FLobbyDataRegistryEOS : public TSharedFromThis<FLobbyDataRegistryEOS>
{
public:
	FLobbyDataRegistryEOS(const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites);

	TSharedPtr<FLobbyDataEOS> Find(EOS_LobbyId LobbyIdEOS) const;
	TSharedPtr<FLobbyDataEOS> Find(FLobbyId LobbyId) const;
	TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyDataEOS>>> FindOrCreateFromLobbyDetails(FAccountId LocalAccountId, const TSharedRef<FLobbyDetailsEOS>& LobbyDetails);

private:
	void Register(const TSharedRef<FLobbyDataEOS>& LobbyIdHandleData);
	void Unregister(FLobbyId LobbyId);
	FLobbyDataEOS::FUnregisterFn MakeUnregisterFn();

	TSharedRef<FLobbyPrerequisitesEOS> Prerequisites;
	TMap<FString, TWeakPtr<FLobbyDataEOS>> LobbyIdIndex;
	TMap<FLobbyId, TWeakPtr<FLobbyDataEOS>> LobbyIdHandleIndex;
	uint32 NextHandleIndex = 1;
};

class FLobbyInviteIdEOS final
{
public:
	FLobbyInviteIdEOS() = default;
	FLobbyInviteIdEOS(const char* InviteIdEOS)
	{
		FPlatformString::Strncpy(Data, InviteIdEOS, MaxLobbyInviteIdSize);
	}

	const char* Get() const { return Data; }

private:
	static const int MaxLobbyInviteIdSize = 256;
	char Data[MaxLobbyInviteIdSize] = {};
};

/**
* Lobby invite data will keep the lobby object valid until the invitation has been accepted or rejected.
*/
class FLobbyInviteDataEOS final
{
public:
	FLobbyInviteDataEOS() = delete;

	static TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbyInviteDataEOS>>> CreateFromInviteId(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		const TSharedRef<FLobbyDataRegistryEOS>& LobbyDataRegistry,
		FAccountId LocalAccountId,
		const char* InviteIdEOS,
		EOS_ProductUserId Sender);

	TSharedRef<FLobbyDetailsEOS> GetLobbyDetails() const { return LobbyDetails; }
	TSharedRef<FLobbyDataEOS> GetLobbyData() const { return LobbyData; }

	const char* GetInviteIdEOS() const { return InviteIdEOS->Get(); }
	const FString& GetInviteId() const { return InviteId; }

	FAccountId GetReceiver() const { return Receiver; }
	FAccountId GetSender() const { return Sender; }

private:
	template <typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FLobbyInviteDataEOS(
		const TSharedRef<FLobbyInviteIdEOS>& InviteIdEOS,
		FAccountId Receiver,
		FAccountId Sender,
		const TSharedRef<FLobbyDetailsEOS>& LobbyDetails,
		const TSharedRef<FLobbyDataEOS>& LobbyData);

	TSharedRef<FLobbyInviteIdEOS> InviteIdEOS;
	FAccountId Receiver;
	FAccountId Sender;
	TSharedRef<FLobbyDetailsEOS> LobbyDetails;
	TSharedRef<FLobbyDataEOS> LobbyData;
	FString InviteId;
};

/**
* The lobby search object is meant to hold the lifetimes of lobby search results so that the
* lobby details will be discoverable during a join operation.
*/
class FLobbySearchEOS final
{
public:
	FLobbySearchEOS() = delete;

	static TFuture<TDefaultErrorResultInternal<TSharedRef<FLobbySearchEOS>>> Create(
		const TSharedRef<FLobbyPrerequisitesEOS>& Prerequisites,
		const TSharedRef<FLobbyDataRegistryEOS>& LobbyRegistry,
		const FLobbySearchParameters& Params);

	TArray<TSharedRef<const FLobby>> GetLobbyResults() const;
	const TArray<TSharedRef<FLobbyDataEOS>>& GetLobbyData();

private:
	class FSearchHandle
	{
	public:
		UE_NONCOPYABLE(FSearchHandle);
		FSearchHandle() = default;

		~FSearchHandle()
		{
			EOS_LobbySearch_Release(SearchHandle);
		}

		EOS_HLobbySearch& Get()
		{
			return SearchHandle;
		}

	private:
		EOS_HLobbySearch SearchHandle = {};
	};

	template <typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
	FLobbySearchEOS(const TSharedRef<FSearchHandle>& SearchHandle, TArray<TSharedRef<FLobbyDataEOS>>&& Lobbies);

	TSharedRef<FSearchHandle> SearchHandle;
	TArray<TSharedRef<FLobbyDataEOS>> Lobbies;
};

FString ToLogString(const FLobbyDataEOS& LobbyData);

inline FString LexToString(const FLobbyDataEOS& LobbyData)
{
	return ToLogString(LobbyData);
}

/* UE::Online */ }
