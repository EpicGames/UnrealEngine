// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineIdEOSGS.h"

#include "EOSShared.h"
#include "String/HexToBytes.h"
#include "String/BytesToHex.h"

namespace UE::Online {

const uint8 OnlineIdEOSUtf8BufferLength = 32;
const uint8 OnlineIdEOSHexBufferLength = 16;

FOnlineAccountIdRegistryEOSGS::FOnlineAccountIdRegistryEOSGS()
	: Registry(EOnlineServices::Epic)
{

}

FOnlineAccountIdRegistryEOSGS::FOnlineAccountIdRegistryEOSGS(EOnlineServices Services)
	: Registry(Services)
{

}

FAccountId FOnlineAccountIdRegistryEOSGS::FindOrAddAccountId(const EOS_ProductUserId ProductUserId)
{
	if (ensure(EOS_ProductUserId_IsValid(ProductUserId)))
	{
		return Registry.FindOrAddHandle(ProductUserId);
	}
	return Registry.GetInvalidHandle();
}

FAccountId FOnlineAccountIdRegistryEOSGS::FindAccountId(const EOS_ProductUserId ProductUserId) const
{
	return Registry.FindHandle(ProductUserId);
}

EOS_ProductUserId FOnlineAccountIdRegistryEOSGS::GetProductUserId(const FAccountId& AccountId) const
{
	return Registry.FindIdValue(AccountId);
}

FString FOnlineAccountIdRegistryEOSGS::ToString(const FAccountId& AccountId) const
{
	FString Result;
	if (Registry.ValidateOnlineId(AccountId))
	{
		EOS_ProductUserId ProductUserId = Registry.FindIdValue(AccountId);
		Result = LexToString(ProductUserId);
	}
	else
	{
		check(!AccountId.IsValid()); // Check we haven't been passed a valid handle for a different EOnlineServices.
		Result = TEXT("Invalid");
	}
	return Result;
}

FString FOnlineAccountIdRegistryEOSGS::ToLogString(const FAccountId& AccountId) const
{
	return ToString(AccountId);
}

TArray<uint8> FOnlineAccountIdRegistryEOSGS::ToReplicationData(const FAccountId& AccountId) const
{
	TArray<uint8> ReplicationData;
	if (Registry.ValidateOnlineId(AccountId))
	{
		EOS_ProductUserId ProductUserId = Registry.FindIdValue(AccountId);
		if (ensure(EOS_ProductUserId_IsValid(ProductUserId)))
		{
			char EosBuffer[EOS_PRODUCTUSERID_MAX_LENGTH + 1] = {};
			int32 EosBufferLength = sizeof(EosBuffer);
			const EOS_EResult EosResult = EOS_ProductUserId_ToString(ProductUserId, EosBuffer, &EosBufferLength);
			if (ensure(EosResult == EOS_EResult::EOS_Success))
			{
				check(EosBufferLength - 1 == OnlineIdEOSUtf8BufferLength);
				ReplicationData.SetNumUninitialized(OnlineIdEOSHexBufferLength);
				UE::String::HexToBytes(FUtf8StringView(EosBuffer, OnlineIdEOSUtf8BufferLength), ReplicationData.GetData());
			}
		}
	}
	return ReplicationData;
}

FAccountId FOnlineAccountIdRegistryEOSGS::FromReplicationData(const TArray<uint8>& ReplicationData)
{
	if (ReplicationData.Num() == OnlineIdEOSHexBufferLength)
	{
		char EosBuffer[EOS_EPICACCOUNTID_MAX_LENGTH + 1] = {};
		UE::String::BytesToHex(TConstArrayView<uint8>(ReplicationData.GetData(), OnlineIdEOSHexBufferLength), EosBuffer);
		EOS_ProductUserId ProductUserId = EOS_ProductUserId_FromString(EosBuffer);
		return FindOrAddAccountId(ProductUserId);
	}
	return Registry.GetInvalidHandle();
}

FAccountId FOnlineAccountIdRegistryEOSGS::FromStringData(const FString& StringData)
{
	const auto Utf8ProductUserId = StringCast<UTF8CHAR>(*StringData);
	EOS_ProductUserId ProductUserId = EOS_ProductUserId_FromString((const char*)Utf8ProductUserId.Get());
	return FindOrAddAccountId(ProductUserId);
}

IOnlineAccountIdRegistryEOSGS& FOnlineAccountIdRegistryEOSGS::GetRegistered()
{
	return FOnlineAccountIdRegistryEOSGS::GetRegistered(EOnlineServices::Epic);
}

IOnlineAccountIdRegistryEOSGS& FOnlineAccountIdRegistryEOSGS::GetRegistered(EOnlineServices Services)
{
	check(Services == EOnlineServices::Epic || Services == EOnlineServices::EpicGame);

	IOnlineAccountIdRegistry* Registry = FOnlineIdRegistryRegistry::Get().GetAccountIdRegistry(Services);
	check(Registry);
	
	return *static_cast<IOnlineAccountIdRegistryEOSGS*>(Registry);
}

EOS_ProductUserId GetProductUserId(const FAccountId& AccountId)
{
	return FOnlineAccountIdRegistryEOSGS::GetRegistered(AccountId.GetOnlineServicesType()).GetProductUserId(AccountId);
}

EOS_ProductUserId GetProductUserIdChecked(const FAccountId& AccountId)
{
	EOS_ProductUserId ProductUserId = GetProductUserId(AccountId);
	check(EOS_ProductUserId_IsValid(ProductUserId) == EOS_TRUE);
	return ProductUserId;
}

FAccountId FindAccountId(const EOS_ProductUserId ProductUserId)
{
	return FindAccountId(EOnlineServices::Epic, ProductUserId);
}

FAccountId FindAccountId(EOnlineServices Services, const EOS_ProductUserId ProductUserId)
{
	return FOnlineAccountIdRegistryEOSGS::GetRegistered(Services).FindAccountId(ProductUserId);
}

FAccountId FindAccountIdChecked(const EOS_ProductUserId ProductUserId)
{
	return FindAccountIdChecked(EOnlineServices::Epic, ProductUserId);
}

FAccountId FindAccountIdChecked(EOnlineServices Services, const EOS_ProductUserId ProductUserId)
{
	FAccountId Result = FindAccountId(Services, ProductUserId);
	check(Result.IsValid());
	return Result;
}

/* UE::Online */ }
