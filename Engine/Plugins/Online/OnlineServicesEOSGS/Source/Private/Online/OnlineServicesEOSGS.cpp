// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEOSGS.h"

#include "Online/AchievementsEOSGS.h"
#include "Online/AuthEOSGS.h"
#include "Online/EpicProductUserIdResolverEOSGS.h"
#include "Online/LeaderboardsEOSGS.h"
#include "Online/LobbiesEOSGS.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/StatsEOSGS.h"
#include "Online/SessionsEOSGS.h"
#include "Online/TitleFileEOSGS.h"
#include "Online/UserFileEOSGS.h"
#include "Online/PlayerReportsEOSGS.h"
#include "Online/PlayerSanctionsEOSGS.h"
#include "IEOSSDKManager.h"

#if WITH_ENGINE
#include "InternetAddrEOS.h"
#include "NetDriverEOS.h"
#include "SocketSubsystemEOS.h"
#include "SocketSubsystemEOSUtils_OnlineServicesEOSGS.h"
#endif

namespace UE::Online {

FOnlineServicesEOSGS::FOnlineServicesEOSGS(FName InInstanceName, FName InInstanceConfigName)
	: Super(GetServiceConfigNameStatic(), InInstanceName, InInstanceConfigName)
{
}

bool FOnlineServicesEOSGS::PreInit()
{
	if(!Super::PreInit())
	{
		return false;
	}
	
#if WITH_ENGINE
	SocketSubsystem = MakeShared<FSocketSubsystemEOS>(EOSPlatformHandle, MakeShared< FSocketSubsystemEOSUtils_OnlineServicesEOS>(*this));
	check(SocketSubsystem);

	FString ErrorStr;
	if (!SocketSubsystem->Init(ErrorStr))
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s] Unable to initialize Socket Subsystem. Error=[%ls]", __FUNCTION__, *ErrorStr);
		return false;
	}
#endif

	return true;
}

void FOnlineServicesEOSGS::Destroy()
{
	Super::Destroy();

#if WITH_ENGINE
	if (SocketSubsystem)
	{
		SocketSubsystem->Shutdown();
		SocketSubsystem = nullptr;
	}
#endif
}

void FOnlineServicesEOSGS::RegisterComponents()
{
	Components.Register<FAchievementsEOSGS>(*this);
	Components.Register<FAuthEOSGS>(*this);
	Components.Register<FEpicProductUserIdResolverEOSGS>(*this);
	Components.Register<FLeaderboardsEOSGS>(*this);
	Components.Register<FLobbiesEOSGS>(*this);
	Components.Register<FPlayerReportsEOSGS>(*this);
	Components.Register<FPlayerSanctionsEOSGS>(*this);
	Components.Register<FStatsEOSGS>(*this);
	Components.Register<FSessionsEOSGS>(*this);
	if (EOS_Platform_GetTitleStorageInterface(*GetEOSPlatformHandle()))
	{
		Components.Register<FTitleFileEOSGS>(*this);
	}
	else
	{
		WarnIfEncryptionKeyMissing(TEXT("TitleStorage"));
	}
	if (EOS_Platform_GetPlayerDataStorageInterface(*GetEOSPlatformHandle()))
	{
		Components.Register<FUserFileEOSGS>(*this);
	}
	else
	{
		WarnIfEncryptionKeyMissing(TEXT("PlayerDataStorage"));
	}
	Super::RegisterComponents();
}

TOnlineResult<FGetResolvedConnectString> FOnlineServicesEOSGS::GetResolvedConnectString(FGetResolvedConnectString::Params&& Params)
{
	if (Params.LobbyId.IsValid())
	{
		ILobbiesPtr LobbiesEOS = GetLobbiesInterface();
		check(LobbiesEOS);

		TOnlineResult<FGetJoinedLobbies> JoinedLobbies = LobbiesEOS->GetJoinedLobbies({ Params.LocalAccountId });
		if (JoinedLobbies.IsOk())
		{
			for (TSharedRef<const FLobby>& Lobby : JoinedLobbies.GetOkValue().Lobbies)
			{
				if (Lobby->LobbyId == Params.LobbyId)
				{
#if WITH_ENGINE
					//It should look like this: "[EOS:0002aeeb5b2d4388a3752dd6d31222ec]"
					FInternetAddrEOS TempAddr(GetProductUserIdChecked(Lobby->OwnerAccountId));
					return TOnlineResult<FGetResolvedConnectString>({ FString::Printf(TEXT("[%s]"), *TempAddr.ToString(true)) });
#else
					return TOnlineResult<FGetResolvedConnectString>(Errors::NotImplemented());
#endif
				}
			}

			// No matching lobby
			return TOnlineResult<FGetResolvedConnectString>(Errors::NotFound());
		}
		else
		{
			return TOnlineResult<FGetResolvedConnectString>(JoinedLobbies.GetErrorValue());
		}
	}
	else if (Params.SessionId.IsValid())
	{
		ISessionsPtr SessionsEOS = GetSessionsInterface();
		check(SessionsEOS);

		TOnlineResult<FGetSessionById> Result = SessionsEOS->GetSessionById({ Params.SessionId });
		if (Result.IsOk())
		{
#if WITH_ENGINE
			TSharedRef<const ISession> Session = Result.GetOkValue().Session;

			//It should look like this: "[EOS:0002aeeb5b2d4388a3752dd6d31222ec]"
			FInternetAddrEOS TempAddr(GetProductUserIdChecked(Session->GetOwnerAccountId()));
			return TOnlineResult<FGetResolvedConnectString>({ FString::Printf(TEXT("[%s]"), *TempAddr.ToString(true)) });
#else
			return TOnlineResult<FGetResolvedConnectString>(Errors::NotImplemented());
#endif
		}
		else
		{
			return TOnlineResult<FGetResolvedConnectString>(Result.GetErrorValue());
		}
	}

	// No valid lobby or session id set
	return TOnlineResult<FGetResolvedConnectString>(Errors::InvalidParams());
}

// Get specific OnlineServiesEOSGS interfaces
IPlayerReportsPtr FOnlineServicesEOSGS::GetPlayerReportsInterface()
{
	return IPlayerReportsPtr(AsShared(), Get<IPlayerReports>());
}

IPlayerSanctionsPtr FOnlineServicesEOSGS::GetPlayerSanctionsInterface()
{
	return IPlayerSanctionsPtr(AsShared(), Get<IPlayerSanctions>());
}

/* UE::Online */ }
