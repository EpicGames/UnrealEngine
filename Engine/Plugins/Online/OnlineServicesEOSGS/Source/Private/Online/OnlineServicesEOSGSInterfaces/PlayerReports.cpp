// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEOSGSInterfaces/PlayerReports.h"

namespace UE::Online {

const TCHAR* LexToString(EPlayerReportCategory Value)
{
	switch (Value)
	{
	case EPlayerReportCategory::Cheating:
		return TEXT("Cheating");
	case EPlayerReportCategory::Exploiting:
		return TEXT("Exploiting");
	case EPlayerReportCategory::OffensiveProfile:
		return TEXT("OffensiveProfile");
	case EPlayerReportCategory::VerbalAbuse:
		return TEXT("VerbalAbuse");
	case EPlayerReportCategory::Scamming:
		return TEXT("Scamming");
	case EPlayerReportCategory::Spamming:
		return TEXT("Spamming");
	default: checkNoEntry();
	case EPlayerReportCategory::Other:
		return TEXT("Other");
	}
}

void LexFromString(EPlayerReportCategory& OutValue, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Cheating")) == 0)
	{
		OutValue = EPlayerReportCategory::Cheating;
	}
	else if (FCString::Stricmp(InStr, TEXT("Exploiting")) == 0)
	{
		OutValue = EPlayerReportCategory::Exploiting; 
	}
	else if (FCString::Stricmp(InStr, TEXT("OffensiveProfile")) == 0)
	{
		OutValue = EPlayerReportCategory::OffensiveProfile;
	}
	else if (FCString::Stricmp(InStr, TEXT("VerbalAbuse")) == 0)
	{
		OutValue = EPlayerReportCategory::VerbalAbuse;
	}
	else if (FCString::Stricmp(InStr, TEXT("Scamming")) == 0)
	{
		OutValue = EPlayerReportCategory::Scamming;
	}
	else if (FCString::Stricmp(InStr, TEXT("Spamming")) == 0)
	{
		OutValue = EPlayerReportCategory::Spamming;
	}
	else
	{
		OutValue = EPlayerReportCategory::Other; 
	}
}

/* UE::Online */}