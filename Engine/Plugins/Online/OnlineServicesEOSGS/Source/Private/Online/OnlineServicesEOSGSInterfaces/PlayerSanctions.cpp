// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEOSGSInterfaces/PlayerSanctions.h"

namespace UE::Online {

const TCHAR* LexToString(EPlayerSanctionAppealReason Value)
{
	switch (Value)
	{
	case EPlayerSanctionAppealReason::IncorrectSanction:
		return TEXT("IncorrectSanction");
	case EPlayerSanctionAppealReason::CompromisedAccount:
		return TEXT("CompromisedAccount");
	case EPlayerSanctionAppealReason::UnfairPunishment:
		return TEXT("UnfairPunishment");
	default: checkNoEntry();
	case EPlayerSanctionAppealReason::AppealForForgiveness:
		return TEXT("AppealForForgiveness");
	}
}

void LexFromString(EPlayerSanctionAppealReason& OutValue, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("IncorrectSanction")) == 0)
	{
		OutValue = EPlayerSanctionAppealReason::IncorrectSanction;
	}
	else if (FCString::Stricmp(InStr, TEXT("CompromisedAccount")) == 0)
	{
		OutValue = EPlayerSanctionAppealReason::CompromisedAccount;
	}
	else if (FCString::Stricmp(InStr, TEXT("UnfairPunishment")) == 0)
	{
		OutValue = EPlayerSanctionAppealReason::UnfairPunishment;
	}
	else if (FCString::Stricmp(InStr, TEXT("AppealForForgiveness")) == 0)
	{
		OutValue = EPlayerSanctionAppealReason::AppealForForgiveness;
	}
}
	
/* UE::Online */}