// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEOSGSModule.h"

#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"
#include "Online/OnlineServicesRegistry.h"
#include "Online/SessionsEOSGS.h"

namespace UE::Online
{

class FOnlineServicesFactoryEOSGS : public IOnlineServicesFactory
{
public:
	virtual ~FOnlineServicesFactoryEOSGS() {}
	virtual TSharedPtr<IOnlineServices> Create(FName InInstanceName, FName InInstanceConfigName) override
	{
		TSharedPtr<FOnlineServicesEOSGS> Result = MakeShared<FOnlineServicesEOSGS>(InInstanceName, InInstanceConfigName);
		if (!Result->PreInit())
		{
			Result = nullptr;
		}
		return Result;
	}
};

int FOnlineServicesEOSGSModule::GetRegistryPriority()
{
	return 0;
}

void FOnlineServicesEOSGSModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("OnlineServicesInterface"));

	if (TEMP_ShouldUseEpicModularPlugins())
	{
		UE_LOGF(LogOnlineServices, Verbose, "TEMP_ShouldUseEpicModularPlugins() = true, skipping FOnlineServicesEOSGSModule startup.");
		return;
	}

	FModuleManager::Get().LoadModuleChecked(TEXT("EOSShared"));

	FOnlineServicesRegistry::Get().RegisterServicesFactory(EOnlineServices::Epic, MakeUnique<FOnlineServicesFactoryEOSGS>(), GetRegistryPriority());

	static FOnlineAccountIdRegistryEOSGS AccountIdRegistry(EOnlineServices::Epic);
	FOnlineIdRegistryRegistry::Get().RegisterAccountIdRegistry(EOnlineServices::Epic, &AccountIdRegistry, GetRegistryPriority());

	static FOnlineSessionIdRegistryEOSGS SessionIdRegistry(EOnlineServices::Epic);
	FOnlineIdRegistryRegistry::Get().RegisterSessionIdRegistry(EOnlineServices::Epic, &SessionIdRegistry, GetRegistryPriority());

	static FOnlineSessionInviteIdRegistryEOSGS SessionInviteIdRegistry(EOnlineServices::Epic);
	FOnlineIdRegistryRegistry::Get().RegisterSessionInviteIdRegistry(EOnlineServices::Epic, &SessionInviteIdRegistry, GetRegistryPriority());
}

void FOnlineServicesEOSGSModule::ShutdownModule()
{
	FOnlineServicesRegistry::Get().UnregisterServicesFactory(EOnlineServices::Epic, GetRegistryPriority());
	FOnlineIdRegistryRegistry::Get().UnregisterAccountIdRegistry(EOnlineServices::Epic, GetRegistryPriority());
	FOnlineIdRegistryRegistry::Get().UnregisterSessionIdRegistry(EOnlineServices::Epic, GetRegistryPriority());
	FOnlineIdRegistryRegistry::Get().UnregisterSessionInviteIdRegistry(EOnlineServices::Epic, GetRegistryPriority());
}

/* UE::Online */ }

IMPLEMENT_MODULE(UE::Online::FOnlineServicesEOSGSModule, OnlineServicesEOSGS);
