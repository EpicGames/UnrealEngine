// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PlayerReportsEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_reports.h"

namespace UE::Online {
	
FPlayerReportsEOSGS::FPlayerReportsEOSGS(FOnlineServicesEOSGS& InServices) : TOnlineComponent(TEXT("PlayerReports"), InServices)
{
}

void FPlayerReportsEOSGS::Initialize()
{
	Super::Initialize(); 

	PlayerReportsHandle = EOS_Platform_GetReportsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(PlayerReportsHandle);

	// No need to register for any EOS notifications/events

	RegisterCommands(); 
}

void FPlayerReportsEOSGS::RegisterCommands()
{
	RegisterCommand(&FPlayerReportsEOSGS::SendPlayerReport);
}

// Convert the plugin player report category enum to the EOS SDK enum
EOS_EPlayerReportsCategory ToEOSPlayerReportsCategory(EPlayerReportCategory Category)
{
	switch (Category)
	{
	case EPlayerReportCategory::Cheating:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Cheating;
	}
	case EPlayerReportCategory::Exploiting:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Exploiting;
	}
	case EPlayerReportCategory::OffensiveProfile:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_OffensiveProfile;
	}
	case EPlayerReportCategory::VerbalAbuse:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_VerbalAbuse;
	}
	case EPlayerReportCategory::Scamming:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Scamming;
	}
	case EPlayerReportCategory::Spamming:
	{
		return EOS_EPlayerReportsCategory::EOS_PRC_Spamming;
	}
	}
	// Default to other
	return EOS_EPlayerReportsCategory::EOS_PRC_Other;
}

TOnlineAsyncOpHandle<FSendPlayerReport> FPlayerReportsEOSGS::SendPlayerReport(FSendPlayerReport::Params&& Params)
{
	TOnlineAsyncOpRef<FSendPlayerReport> Op = GetOp<FSendPlayerReport>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FSendPlayerReport>& InAsyncOp)
		{
			const FSendPlayerReport::Params& Params = InAsyncOp.GetParams();

			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FSendPlayerReport>& InAsyncOp, TPromise<const EOS_Reports_SendPlayerBehaviorReportCompleteCallbackInfo*>&& Promise)
		{
			const FSendPlayerReport::Params& Params = InAsyncOp.GetParams();

			EOS_Reports_SendPlayerBehaviorReportOptions Options = {}; 
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_REPORTS_SENDPLAYERBEHAVIORREPORT_API_LATEST, 2);
			Options.ReporterUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.ReportedUserId = GetProductUserIdChecked(Params.TargetAccountId);
			Options.Category = ToEOSPlayerReportsCategory(Params.Category);
			const auto MessagedUTF8 = StringCast<UTF8CHAR>(*Params.Message);
			Options.Message =Params.Message.IsEmpty() ? nullptr : (const char*)MessagedUTF8.Get();
			const auto ContextUTF8 = StringCast<UTF8CHAR>(*Params.Context);
			Options.Context = Params.Context.IsEmpty() ? nullptr : (const char*)ContextUTF8.Get();

			EOS_Async(EOS_Reports_SendPlayerBehaviorReport, PlayerReportsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FSendPlayerReport>& InAsyncOp, const EOS_Reports_SendPlayerBehaviorReportCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Reports_SendPlayerBehaviorReport result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "EOS_Reports_SendPlayerBehaviorReport result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetResult({});
			}
		}).Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

FString ToLogString(const FSendPlayerReport::Result& SendPlayerReportResult)
{
	// SendPlayerReport Result is empty 
	return FString();
}
/* UE::Online */ }