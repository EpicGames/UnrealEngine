// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PlayerSanctionsEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_sanctions.h"

namespace UE::Online {
	
FPlayerSanctionsEOSGS::FPlayerSanctionsEOSGS(FOnlineServicesEOSGS& InServices) : TOnlineComponent(TEXT("PlayerSanctions"), InServices)
{
}

void FPlayerSanctionsEOSGS::Initialize()
{
	Super::Initialize(); 

	PlayerSanctionsHandle = EOS_Platform_GetSanctionsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(PlayerSanctionsHandle);

	// No need to register for any EOS notifications/events

	RegisterCommands();
}

void FPlayerSanctionsEOSGS::RegisterCommands()
{
	RegisterCommand(&FPlayerSanctionsEOSGS::CreatePlayerSanctionAppeal);
	RegisterCommand(&FPlayerSanctionsEOSGS::ReadEntriesForUser);
}

// Convert the plugin player sanction category enum to the EOS SDK enum
EOS_ESanctionAppealReason ToEOSSanctionAppealReason(EPlayerSanctionAppealReason Reason)
{
	switch (Reason)
	{
	case EPlayerSanctionAppealReason::IncorrectSanction:
		return EOS_ESanctionAppealReason::EOS_SAR_IncorrectSanction;
	case EPlayerSanctionAppealReason::CompromisedAccount:
		return EOS_ESanctionAppealReason::EOS_SAR_CompromisedAccount;
	case EPlayerSanctionAppealReason::UnfairPunishment:
		return EOS_ESanctionAppealReason::EOS_SAR_UnfairPunishment;
	case EPlayerSanctionAppealReason::AppealForForgiveness:
		return EOS_ESanctionAppealReason::EOS_SAR_AppealForForgiveness;
	}
	return EOS_ESanctionAppealReason::EOS_SAR_Invalid;
}

TOnlineAsyncOpHandle<FCreatePlayerSanctionAppeal> FPlayerSanctionsEOSGS::CreatePlayerSanctionAppeal(FCreatePlayerSanctionAppeal::Params&& Params)
{
	TOnlineAsyncOpRef<FCreatePlayerSanctionAppeal> Op = GetOp<FCreatePlayerSanctionAppeal>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FCreatePlayerSanctionAppeal>& InAsyncOp)
		{
			const FCreatePlayerSanctionAppeal::Params& Params = InAsyncOp.GetParams();

			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
					InAsyncOp.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FCreatePlayerSanctionAppeal>& InAsyncOp, TPromise<const EOS_Sanctions_CreatePlayerSanctionAppealCallbackInfo*>&& Promise)
		{
			const FCreatePlayerSanctionAppeal::Params& Params = InAsyncOp.GetParams();

			EOS_Sanctions_CreatePlayerSanctionAppealOptions Options = {}; 
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_CREATEPLAYERSANCTIONAPPEAL_API_LATEST, 1);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.Reason = ToEOSSanctionAppealReason(Params.Reason);
			const auto ReferenceIdUTF8 = StringCast<UTF8CHAR>(*Params.ReferenceId);
			Options.ReferenceId = (const char*)ReferenceIdUTF8.Get();

			EOS_Async(EOS_Sanctions_CreatePlayerSanctionAppeal, PlayerSanctionsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FCreatePlayerSanctionAppeal>& InAsyncOp, const EOS_Sanctions_CreatePlayerSanctionAppealCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Sanctions_CreatePlayerSanctionAppeal result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "EOS_Sanctions_CreatePlayerSanctionAppeal result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetResult({});
			}
		}).Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FReadActivePlayerSanctions> FPlayerSanctionsEOSGS::ReadEntriesForUser(FReadActivePlayerSanctions::Params&& Params)
{
	TOnlineAsyncOpRef<FReadActivePlayerSanctions> Op = GetOp<FReadActivePlayerSanctions>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FReadActivePlayerSanctions>& InAsyncOp)
		{
			const FReadActivePlayerSanctions::Params& Params = InAsyncOp.GetParams();

			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FReadActivePlayerSanctions>& InAsyncOp, TPromise<const EOS_Sanctions_QueryActivePlayerSanctionsCallbackInfo*>&& Promise)
		{
			const FReadActivePlayerSanctions::Params& Params = InAsyncOp.GetParams();

			EOS_Sanctions_QueryActivePlayerSanctionsOptions Options = {}; 
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_QUERYACTIVEPLAYERSANCTIONS_API_LATEST, 2);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);

			EOS_Async(EOS_Sanctions_QueryActivePlayerSanctions, PlayerSanctionsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FReadActivePlayerSanctions>& InAsyncOp, const EOS_Sanctions_QueryActivePlayerSanctionsCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Sanctions_QueryActivePlayerSanctions result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			UE_LOGF(LogOnlineServices, Verbose, "EOS_Sanctions_QueryActivePlayerSanctions result=[%ls]", *LexToString(Data->ResultCode));

			const FReadActivePlayerSanctions::Params& Params = InAsyncOp.GetParams();

			EOS_Sanctions_GetPlayerSanctionCountOptions CountOptions = {};
			CountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_GETPLAYERSANCTIONCOUNT_API_LATEST, 1);
			CountOptions.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);

			uint32 SanctionCount = EOS_Sanctions_GetPlayerSanctionCount(PlayerSanctionsHandle, &CountOptions);

			EOS_Sanctions_CopyPlayerSanctionByIndexOptions CopyOptions = {};
			CopyOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SANCTIONS_COPYPLAYERSANCTIONBYINDEX_API_LATEST, 1);
			CopyOptions.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);

			FReadActivePlayerSanctions::Result Result; 

			for (uint32 SanctionIndex = 0; SanctionIndex < SanctionCount; SanctionIndex++)
			{
				CopyOptions.SanctionIndex = SanctionIndex;

				EOS_Sanctions_PlayerSanction* PlayerSanctionEOS = nullptr;

				EOS_EResult CopyResult = EOS_Sanctions_CopyPlayerSanctionByIndex(PlayerSanctionsHandle, &CopyOptions, &PlayerSanctionEOS);
				if (CopyResult == EOS_EResult::EOS_Success)
				{
					FActivePlayerSanctionEntry& Entry = Result.Entries.Emplace_GetRef(); 
					Entry.TimePlaced = PlayerSanctionEOS->TimePlaced;
					Entry.TimeExpires = PlayerSanctionEOS->TimeExpires;
					Entry.ReferenceId = PlayerSanctionEOS->ReferenceId;
					Entry.Action = PlayerSanctionEOS->Action;

					EOS_Sanctions_PlayerSanction_Release(PlayerSanctionEOS);
				}
				else
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_Sanctions_CopyPlayerSanctionByIndex result=[%ls]", *LexToString(Data->ResultCode));
				}
			}

			ToLogString(Result); 
									
			InAsyncOp.SetResult(MoveTemp(Result));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

FString ToLogString(const FCreatePlayerSanctionAppeal::Result& CreatePlayerSanctionAppealResult)
{
	// CreatePlayerSanctionAppeal result is empty
	return FString();
}

FString ToLogString(const FReadActivePlayerSanctions::Result& ReadPlayerSanctionResult)
{
	TArray<FString> StringEntries;

	for (FActivePlayerSanctionEntry Entry : ReadPlayerSanctionResult.Entries)
	{
		StringEntries.Emplace(FString::Printf(TEXT("ReferenceId:%s Action:%s TimeExpires:%llu TimePlaced:%llu"), *Entry.ReferenceId, *Entry.Action, Entry.TimeExpires, Entry.TimePlaced));
	}

	return FString::Join(StringEntries, TEXT(","));
}
/* UE::Online */ }