// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SessionsEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/EpicProductUserIdResolver.h"
#include "Online/LobbiesCommonTypes.h"
#include "Online/NboSerializerEOSGSSvc.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineServicesEOSGS.h"
#include "Online/SessionsEOSGSTypes.h"

#include "eos_sessions.h"

namespace UE::Online {

/** FOnlineSessionIdRegistryEOSGS */

FOnlineSessionIdRegistryEOSGS::FOnlineSessionIdRegistryEOSGS(EOnlineServices ServicesType)
	: FOnlineSessionIdRegistryLAN(ServicesType)
{
}

FOnlineSessionIdRegistryEOSGS& FOnlineSessionIdRegistryEOSGS::GetRegistered(EOnlineServices Services)
{
	IOnlineSessionIdRegistry* Registry = FOnlineIdRegistryRegistry::Get().GetSessionIdRegistry(Services);
	check(Registry);
	return *static_cast<FOnlineSessionIdRegistryEOSGS*>(Registry);
}

/** FOnlineSessionInviteIdRegistryEOSGS */

FOnlineSessionInviteIdRegistryEOSGS::FOnlineSessionInviteIdRegistryEOSGS(EOnlineServices ServicesType)
	: FOnlineSessionInviteIdStringRegistry(ServicesType)
{
}

FOnlineSessionInviteIdRegistryEOSGS& FOnlineSessionInviteIdRegistryEOSGS::GetRegistered(EOnlineServices Services)
{
	IOnlineSessionInviteIdRegistry* Registry = FOnlineIdRegistryRegistry::Get().GetSessionInviteIdRegistry(Services);
	check(Registry);
	return *static_cast<FOnlineSessionInviteIdRegistryEOSGS*>(Registry);
}

/** FSessionEOSGS */

FSessionEOSGS::FSessionEOSGS(const FSessionsEOSGS& SessionsInstance, const TSharedPtr<FSessionDetailsHandleEOSGS>& InSessionDetailsHandle)
	: SessionDetailsHandle(InSessionDetailsHandle)
{
	EOS_SessionDetails_CopyInfoOptions CopyInfoOptions = { };
	CopyInfoOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_COPYINFO_API_LATEST, 1);

	EOS_SessionDetails_Info* SessionDetailsInfo = nullptr;
	EOS_EResult CopyInfoResult = EOS_SessionDetails_CopyInfo(SessionDetailsHandle->SessionDetailsHandle, &CopyInfoOptions, &SessionDetailsInfo);
	if (CopyInfoResult == EOS_EResult::EOS_Success)
	{
		SessionInfo.SessionId = SessionsInstance.CreateSessionId(UTF8_TO_TCHAR(SessionDetailsInfo->SessionId));
		SessionSettings.CustomSettings.Emplace(EOSGS_HOST_ADDRESS_ATTRIBUTE_KEY, FCustomSessionSetting{ FSchemaVariant(FString(UTF8_TO_TCHAR(SessionDetailsInfo->HostAddress))), ESchemaAttributeVisibility::Public });

		// bInvitesAllowed and bAllowJoinInProgress should both have the same value, and that value corresponds to bAllowNewMembers
		SessionInfo.bAllowSanctionedPlayers = !SessionDetailsInfo->Settings->bSanctionsEnabled;
		SessionSettings.bAllowNewMembers = SessionDetailsInfo->Settings->bInvitesAllowed && SessionDetailsInfo->Settings->bAllowJoinInProgress;
		SessionSettings.NumMaxConnections = SessionDetailsInfo->Settings->NumPublicConnections;
		SessionSettings.JoinPolicy = FromServiceType(SessionDetailsInfo->Settings->PermissionLevel);
		SessionSettings.CustomSettings.Emplace(EOSGS_BUCKET_ID_ATTRIBUTE_KEY, FCustomSessionSetting{ FSchemaVariant(FString(UTF8_TO_TCHAR(SessionDetailsInfo->Settings->BucketId))), ESchemaAttributeVisibility::Public });

		// We retrieve all the session attributes
		EOS_SessionDetails_GetSessionAttributeCountOptions GetAttributeCountOptions = {};
		GetAttributeCountOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_GETSESSIONATTRIBUTECOUNT_API_LATEST, 1);

		uint32_t AttributeCount = EOS_SessionDetails_GetSessionAttributeCount(SessionDetailsHandle->SessionDetailsHandle, &GetAttributeCountOptions);
		for (uint32_t Index = 0; Index < AttributeCount; ++Index)
		{
			// We parse a single attribute

			EOS_SessionDetails_CopySessionAttributeByIndexOptions CopyAttributeByIndexOptions = {};
			CopyAttributeByIndexOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_COPYSESSIONATTRIBUTEBYINDEX_API_LATEST, 1);

			CopyAttributeByIndexOptions.AttrIndex = Index;

			EOS_SessionDetails_Attribute* Attribute = nullptr;
			EOS_EResult CopyAttributeByIndexResult = EOS_SessionDetails_CopySessionAttributeByIndex(SessionDetailsHandle->SessionDetailsHandle, &CopyAttributeByIndexOptions, &Attribute);
			if (CopyAttributeByIndexResult == EOS_EResult::EOS_Success)
			{
				FString Key(Attribute->Data->Key);
				FSessionAttributeConverter<ESessionAttributeConversionType::FromService> CustomSettingConverter(*Attribute->Data);
				TPair<FSchemaAttributeId, FSchemaVariant> CustomSettingData = CustomSettingConverter.GetAttributeData();
					
				// Most Session Settings values get parsed in the same way as Custom Session Settings, so we will attempt to retrieve them
				if (Key == EOSGS_ALLOW_NEW_MEMBERS_ATTRIBUTE_KEY.ToString())
				{
					SessionSettings.bAllowNewMembers = CustomSettingData.Value.GetBoolean();
				}
				else if (Key == EOSGS_ANTI_CHEAT_PROTECTED_ATTRIBUTE_KEY.ToString())
				{
					SessionInfo.bAntiCheatProtected = CustomSettingData.Value.GetBoolean();
				}
				else if (Key == EOSGS_IS_DEDICATED_SERVER_SESSION_ATTRIBUTE_KEY.ToString())
				{
					SessionInfo.bIsDedicatedServerSession = CustomSettingData.Value.GetBoolean();
				}
				else if (Key == EOSGS_SCHEMA_NAME_ATTRIBUTE_KEY.ToString())
				{
					SessionSettings.SchemaName = FSchemaId(CustomSettingData.Value.GetString());
				}
				else // The rest are parsed as a Custom Session Setting
				{
					FCustomSessionSetting CustomSessionSetting;
					CustomSessionSetting.Visibility = FromServiceType(Attribute->AdvertisementType);
					CustomSessionSetting.Data = CustomSettingData.Value;

					SessionSettings.CustomSettings.Emplace(Key, CustomSessionSetting);
				}
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "[FSessionEOSGS] EOS_SessionDetails_CopySessionAttributeByIndex failed with result [%ls]", *LexToString(CopyInfoResult));
			}
		}
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionEOSGS] EOS_SessionDetails_CopyInfo failed with result [%ls]", *LexToString(CopyInfoResult));
	}
}

const FSessionEOSGS& FSessionEOSGS::Cast(const ISession& InSession)
{
	check(InSession.GetSessionId().GetOnlineServicesType() == EOnlineServices::Epic);

	return static_cast<const FSessionEOSGS&>(InSession);
}

/** FSessionsEOSGS */

FSessionsEOSGS::FSessionsEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

void FSessionsEOSGS::Initialize()
{
	Super::Initialize();

	SessionsHandle = EOS_Platform_GetSessionsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(SessionsHandle);

	RegisterEventHandlers();
}

void FSessionsEOSGS::Shutdown()
{
	Super::Shutdown();

	UnregisterEventHandlers();
}

void FSessionsEOSGS::RegisterEventHandlers()
{
	// Register for session invites received events
	OnSessionInviteReceivedEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		SessionsHandle,
		1,
		&EOS_Sessions_AddNotifySessionInviteReceived,
		&EOS_Sessions_RemoveNotifySessionInviteReceived,
		&FSessionsEOSGS::HandleSessionInviteReceived);
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYSESSIONINVITERECEIVED_API_LATEST, 1);

	// Register for session invites accepted events
	OnSessionInviteAcceptedEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		SessionsHandle,
		1,
		&EOS_Sessions_AddNotifySessionInviteAccepted,
		&EOS_Sessions_RemoveNotifySessionInviteAccepted,
		&FSessionsEOSGS::HandleSessionInviteAccepted);
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYSESSIONINVITEACCEPTED_API_LATEST, 1);

	// Register for join session accepted events
	OnJoinSessionAcceptedEventRegistration = EOS_RegisterComponentEventHandler(
		this,
		SessionsHandle,
		1,
		&EOS_Sessions_AddNotifyJoinSessionAccepted,
		&EOS_Sessions_RemoveNotifyJoinSessionAccepted,
		&FSessionsEOSGS::HandleJoinSessionAccepted);
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ADDNOTIFYJOINSESSIONACCEPTED_API_LATEST, 1);
}

void FSessionsEOSGS::UnregisterEventHandlers()
{
	OnSessionInviteReceivedEventRegistration = nullptr;
	OnSessionInviteAcceptedEventRegistration = nullptr;
	OnJoinSessionAcceptedEventRegistration = nullptr;
}

void FSessionsEOSGS::HandleSessionInviteReceived(const EOS_Sessions_SessionInviteReceivedCallbackInfo* Data)
{
	FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (LocalAccountId.IsValid())
	{
		FString InviteId(UTF8_TO_TCHAR(Data->InviteId));

		BuildSessionFromInvite(LocalAccountId, InviteId)
			.OnComplete([this, WeakThis = AsWeak(), SenderPUID = Data->TargetUserId, InviteId](const TOnlineResult<FBuildSessionFromDetailsHandle>& Result)
		{
			if (Result.IsOk())
			{
				if (const TSharedPtr<ISessions> StrongThis = WeakThis.Pin())
				{
					// First and second place in the array will be occupied by the receiver and the sender, respectively, since the same order is kept in the array of resolved ids
					const FAccountId& ReceiverId = Result.GetOkValue().LocalAccountId;
					const FAccountId& SenderId = FindAccountId(SenderPUID);

					FSessionInviteId SessionInviteId = CreateSessionInviteId(InviteId);

					TSharedRef<FSessionCommon> Session = Result.GetOkValue().Session;

					TSharedRef<FSessionInviteCommon> SessionInviteRef = MakeShared<FSessionInviteCommon>(
						ReceiverId,
						SenderId,
						SessionInviteId,
						Session->GetSessionId()
					);

					AddSessionInvite(SessionInviteRef, Session, ReceiverId);

					FSessionInviteReceived Event = { ReceiverId, SessionInviteId };

					SessionEvents.OnSessionInviteReceived.Broadcast(Event);
				}
			}

			// We won't broadcast the event if there was an error retrieving the session information
		});
	}
}

void FSessionsEOSGS::HandleSessionInviteAccepted(const EOS_Sessions_SessionInviteAcceptedCallbackInfo* Data)
{
	FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (LocalAccountId.IsValid())
	{
		FString InviteId(UTF8_TO_TCHAR(Data->InviteId));

		BuildSessionFromInvite(LocalAccountId, InviteId)
			.OnComplete([this, WeakThis = AsWeak()](const TOnlineResult<FBuildSessionFromDetailsHandle>& Result)
		{
			if (Result.IsOk())
			{
				if (const TSharedPtr<ISessions> StrongThis = WeakThis.Pin())
				{
					// Instead of using the session information we have stored, we'll use this opportunity to get updated data for the session
					AllSessionsById.Emplace(Result.GetOkValue().Session->GetSessionId(), Result.GetOkValue().Session);

					FUISessionJoinRequested Event{
									Result.GetOkValue().LocalAccountId,
									TResult<FOnlineSessionId, FOnlineError>(Result.GetOkValue().Session->GetSessionId()),
						EUISessionJoinRequestedSource::FromInvitation
					};

					SessionEvents.OnUISessionJoinRequested.Broadcast(Event);

					// The game can react to the OnUISessionJoinRequested event by starting the JoinSession process
				}
			}
		});
	}
}

void FSessionsEOSGS::HandleJoinSessionAccepted(const EOS_Sessions_JoinSessionAcceptedCallbackInfo* Data)
{
	FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (LocalAccountId.IsValid())
	{
		BuildSessionFromUIEvent(LocalAccountId, Data->UiEventId)
			.OnComplete([this, WeakThis = AsWeak()](const TOnlineResult<FBuildSessionFromDetailsHandle>& Result)
		{
			if (Result.IsOk())
			{
				if (const TSharedPtr<ISessions> StrongThis = WeakThis.Pin())
				{
					// Instead of using the session information we have stored, we'll use this opportunity to get updated data for the session
					AllSessionsById.Emplace(Result.GetOkValue().Session->GetSessionId(), Result.GetOkValue().Session);

					FUISessionJoinRequested Event{
						Result.GetOkValue().LocalAccountId,
						TResult<FOnlineSessionId, FOnlineError>(Result.GetOkValue().Session->GetSessionId()),
			EUISessionJoinRequestedSource::Unspecified
					};

					SessionEvents.OnUISessionJoinRequested.Broadcast(Event);

					// The game can react to the OnUISessionJoinRequested event by starting the JoinSession process
				}
			}
		});
	}
}

TOnlineResult<FSetPresenceSession> FSessionsEOSGS::SetPresenceSession(FSetPresenceSession::Params&& Params)
{
	return TOnlineResult<FSetPresenceSession>(Errors::NotImplemented());
}

TOnlineResult<FClearPresenceSession> FSessionsEOSGS::ClearPresenceSession(FClearPresenceSession::Params&& Params)
{
	return TOnlineResult<FClearPresenceSession>(Errors::NotImplemented());
}

TFuture<TOnlineResult<FCreateSession>> FSessionsEOSGS::CreateSessionImpl(const FCreateSession::Params& Params)
{
	// LAN Sessions
	if (Params.bIsLANSession)
	{
		return FSessionsLAN::CreateSessionImpl(Params);
	}

	// EOSGS Sessions

	TPromise<TOnlineResult<FCreateSession>> Promise;
	TFuture<TOnlineResult<FCreateSession>> Future = Promise.GetFuture();

	EOS_Sessions_CreateSessionModificationOptions CreateSessionModificationOptions = {};
	CreateSessionModificationOptions.ApiVersion = 5;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_CREATESESSIONMODIFICATION_API_LATEST, 5);

	CreateSessionModificationOptions.bPresenceEnabled = Params.bPresenceEnabled;
	CreateSessionModificationOptions.bSanctionsEnabled = !Params.bAllowSanctionedPlayers;

	// Check if the Bucket Id custom setting is set, set default otherwise. EOS Sessions can not be created without it
	FString BucketIdStr;
	if (const FCustomSessionSetting* BucketIdSetting = Params.SessionSettings.CustomSettings.Find(EOSGS_BUCKET_ID_ATTRIBUTE_KEY))
	{
		BucketIdStr = BucketIdSetting->Data.GetString();
	}
	else
	{
		int32 BuildUniqueId = GetBuildUniqueId();

		UE_LOGF(LogOnlineServices, Verbose, "[FSessionsEOSGS::CreateSession] 'EOSGS_BUCKET_ID_ATTRIBUTE_KEY' (FString) Custom Setting needed to create EOS sessions not found. Setting \"%d\" as default.", BuildUniqueId);

		BucketIdStr = FString::FromInt(BuildUniqueId);
	}

	const FTCHARToUTF8 BucketIdUtf8(*BucketIdStr);
	CreateSessionModificationOptions.BucketId = BucketIdUtf8.Get();
	
	CreateSessionModificationOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
	CreateSessionModificationOptions.MaxPlayers = Params.SessionSettings.NumMaxConnections;

	const FTCHARToUTF8 SessionIdUtf8(*Params.SessionIdOverride);
	if (SessionIdUtf8.Length())
	{
		CreateSessionModificationOptions.SessionId = SessionIdUtf8.Get();
	}

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	CreateSessionModificationOptions.SessionName = SessionNameUtf8.Get();

	EOS_HSessionModification SessionModificationHandle = nullptr;
	EOS_EResult ResultCode = EOS_Sessions_CreateSessionModification(SessionsHandle, &CreateSessionModificationOptions, &SessionModificationHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Error, "[FSessionsEOSGS::CreateSession] EOS_Sessions_CreateSessionModification failed with result [%ls]", *LexToString(ResultCode));

		Promise.EmplaceValue(Errors::FromEOSResult(ResultCode));

		return Future;
	}

	// We write all SessionSettings values into the SessionModificationHandle
	WriteCreateSessionModificationHandle(SessionModificationHandle, Params);

	// Always update joinability on session creation
	FUpdateSessionImplEOSGS::Params UpdateSessionImplParams{ MakeShared<FSessionModificationHandleEOSGS>(SessionModificationHandle), FUpdateSessionJoinabilityParams{ Params.SessionName, Params.SessionSettings.bAllowNewMembers } };
		
	UpdateSessionImplEOSGS(MoveTemp(UpdateSessionImplParams))
		.Next([this, Promise = MoveTemp(Promise), Params](TDefaultErrorResult<FUpdateSessionImplEOSGS>&& Result) mutable
			{
				if (Result.IsOk())
				{
					// If the session is created successfully, we'll create the local session object
					TSharedRef<FSessionEOSGS> NewSessionEOSGSRef = MakeShared<FSessionEOSGS>();
					NewSessionEOSGSRef->OwnerAccountId = Params.LocalAccountId;
					NewSessionEOSGSRef->SessionSettings = Params.SessionSettings;
					NewSessionEOSGSRef->SessionInfo.SessionId = CreateSessionId(Result.GetOkValue().NewSessionId);
					NewSessionEOSGSRef->SessionInfo.bAllowSanctionedPlayers = Params.bAllowSanctionedPlayers;
					NewSessionEOSGSRef->SessionInfo.bAntiCheatProtected = Params.bAntiCheatProtected;
					NewSessionEOSGSRef->SessionInfo.bIsDedicatedServerSession = IsRunningDedicatedServer();
					NewSessionEOSGSRef->SessionInfo.bIsLANSession = false;
					NewSessionEOSGSRef->SessionInfo.SessionIdOverride = Params.SessionIdOverride;

					AddSessionWithReferences(NewSessionEOSGSRef, Params.SessionName, Params.LocalAccountId, Params.bPresenceEnabled);

					Promise.EmplaceValue(FCreateSession::Result({ }));
				}
				else
				{
					Promise.EmplaceValue(Result.GetErrorValue());
				}
			});

	return Future;
}

TOptional<FOnlineError> FSessionsEOSGS::CheckState(const FCreateSession::Params& Params) const
{
	if (TOptional<FOnlineError> BaseCheck = Super::CheckState(Params))
	{
		return BaseCheck;
	}
	
	if (!Params.bIsLANSession && !Params.SessionIdOverride.IsEmpty())
	{
		int32 Length = Params.SessionIdOverride.Len();

		if (Length < EOS_SESSIONMODIFICATION_MIN_SESSIONIDOVERRIDE_LENGTH || Length > EOS_SESSIONMODIFICATION_MAX_SESSIONIDOVERRIDE_LENGTH)
		{
			UE_LOGF(LogOnlineServices, Warning, "[%ls] Could not create session with SessionIdOverride [%ls] of size [%d]. SessionIdOverride size must be between [%d] and [%d] characters long", UTF8_TO_TCHAR(__FUNCTION__), *Params.SessionIdOverride, Length, EOS_SESSIONMODIFICATION_MIN_SESSIONIDOVERRIDE_LENGTH, EOS_SESSIONMODIFICATION_MAX_SESSIONIDOVERRIDE_LENGTH);

			return TOptional<FOnlineError>(Errors::InvalidParams());
		}
	}

	return TOptional<FOnlineError>();
}

void FSessionsEOSGS::SetHostAddress(EOS_HSessionModification& SessionModHandle, FString HostAddress)
{
	EOS_SessionModification_SetHostAddressOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETHOSTADDRESS_API_LATEST, 1);

	const FTCHARToUTF8 HostAddressUtf8(*HostAddress);
	Options.HostAddress = HostAddressUtf8.Get();

	EOS_EResult ResultCode = EOS_SessionModification_SetHostAddress(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "EOS_SessionModification_SetHostAddress failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetJoinInProgressAllowed(EOS_HSessionModification& SessionModHandle, bool bIsJoinInProgressAllowed)
{
	EOS_SessionModification_SetJoinInProgressAllowedOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETJOININPROGRESSALLOWED_API_LATEST, 1);

	Options.bAllowJoinInProgress = bIsJoinInProgressAllowed;

	EOS_EResult ResultCode = EOS_SessionModification_SetJoinInProgressAllowed(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "EOS_SessionModification_SetJoinInProgressAllowed failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetInvitesAllowed(EOS_HSessionModification& SessionModHandle, bool bAreInvitesAllowed)
{
	EOS_SessionModification_SetInvitesAllowedOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETINVITESALLOWED_API_LATEST, 1);

	Options.bInvitesAllowed = bAreInvitesAllowed;

	EOS_EResult ResultCode = EOS_SessionModification_SetInvitesAllowed(SessionModHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "EOS_SessionModification_SetInvitesAllowed failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetPermissionLevel(EOS_HSessionModification& SessionModificationHandle, const ESessionJoinPolicy& NewJoinPolicy)
{
	EOS_SessionModification_SetPermissionLevelOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETPERMISSIONLEVEL_API_LATEST, 1);

	Options.PermissionLevel = ToServiceType(NewJoinPolicy);

	EOS_EResult ResultCode = EOS_SessionModification_SetPermissionLevel(SessionModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetPermissionLevel] EOS_SessionModification_SetPermissionLevel failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetBucketId(EOS_HSessionModification& SessionModificationHandle, const FString& NewBucketId)
{
	EOS_SessionModification_SetBucketIdOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETBUCKETID_API_LATEST, 1);

	const FTCHARToUTF8 BucketIdUtf8(*NewBucketId);
	Options.BucketId = BucketIdUtf8.Get();

	EOS_EResult ResultCode = EOS_SessionModification_SetBucketId(SessionModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetBucketId] EOS_SessionModification_SetBucketId failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetMaxPlayers(EOS_HSessionModification& SessionModificationHandle, const uint32& NewMaxPlayers)
{
	EOS_SessionModification_SetMaxPlayersOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_SETMAXPLAYERS_API_LATEST, 1);

	Options.MaxPlayers = NewMaxPlayers;

	EOS_EResult ResultCode = EOS_SessionModification_SetMaxPlayers(SessionModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetMaxPlayers] EOS_SessionModification_SetMaxPlayers failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::AddAttribute(EOS_HSessionModification& SessionModificationHandle, const FSchemaAttributeId& Key, const FCustomSessionSetting& Value)
{
	EOS_SessionModification_AddAttributeOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_ADDATTRIBUTE_API_LATEST, 2);

	Options.AdvertisementType = ToServiceType(Value.Visibility);

	FSessionAttributeConverter<ESessionAttributeConversionType::ToService> SessionAttribute(Key, Value.Data);
	Options.SessionAttribute = &SessionAttribute.GetAttributeData();

	EOS_EResult ResultCode = EOS_SessionModification_AddAttribute(SessionModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::AddAttribute] EOS_SessionModification_AddAttribute failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::RemoveAttribute(EOS_HSessionModification& SessionModificationHandle, const FSchemaAttributeId& Key)
{
	EOS_SessionModification_RemoveAttributeOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONMODIFICATION_REMOVEATTRIBUTE_API_LATEST, 1);

	const FTCHARToUTF8 KeyUtf8(Key.ToString());
	Options.Key = KeyUtf8.Get();

	EOS_EResult ResultCode = EOS_SessionModification_RemoveAttribute(SessionModificationHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::RemoveAttribute] EOS_SessionModification_RemoveAttribute failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::WriteCreateSessionModificationHandle(EOS_HSessionModification& SessionModificationHandle, const FCreateSession::Params& Params)
{
	// We'll enable setting a custom host address if a value for it was set as a custom parameter
	if (const FCustomSessionSetting* HostAddressSetting = Params.SessionSettings.CustomSettings.Find(EOSGS_HOST_ADDRESS_ATTRIBUTE_KEY))
	{
		FString HostAddressStr = HostAddressSetting->Data.GetString();
		if (!HostAddressStr.IsEmpty())
		{
			SetHostAddress(SessionModificationHandle, HostAddressStr);
		}
	}

	// We'll update this setting in the session modification step, and start or end the session accordingly to bock or allow join processes
	SetJoinInProgressAllowed(SessionModificationHandle, Params.SessionSettings.bAllowNewMembers);

	// We'll also update invite permissions for the session
	SetInvitesAllowed(SessionModificationHandle, Params.SessionSettings.bAllowNewMembers);

	// We won't copy bIsLANSession since it' irrelevant for EOS Sessions
	AddAttribute(SessionModificationHandle, EOSGS_ALLOW_NEW_MEMBERS_ATTRIBUTE_KEY, { FSchemaVariant(Params.SessionSettings.bAllowNewMembers), ESchemaAttributeVisibility::Public });
	AddAttribute(SessionModificationHandle, EOSGS_ANTI_CHEAT_PROTECTED_ATTRIBUTE_KEY, { FSchemaVariant(Params.bAntiCheatProtected), ESchemaAttributeVisibility::Public });
	AddAttribute(SessionModificationHandle, EOSGS_IS_DEDICATED_SERVER_SESSION_ATTRIBUTE_KEY, { FSchemaVariant(IsRunningDedicatedServer()), ESchemaAttributeVisibility::Public });

	SetPermissionLevel(SessionModificationHandle, Params.SessionSettings.JoinPolicy);

	SetMaxPlayers(SessionModificationHandle, Params.SessionSettings.NumMaxConnections);

	AddAttribute(SessionModificationHandle, EOSGS_SCHEMA_NAME_ATTRIBUTE_KEY, { FSchemaVariant(Params.SessionSettings.SchemaName.ToString()), ESchemaAttributeVisibility::Public });

	// Custom Settings

 	for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : Params.SessionSettings.CustomSettings)
 	{
		AddAttribute(SessionModificationHandle, Entry.Key, Entry.Value);
	}

	// BucketId has its own set method on the API
	const FCustomSessionSetting* NewBucketIdSetting = Params.SessionSettings.CustomSettings.Find(EOSGS_BUCKET_ID_ATTRIBUTE_KEY);
	if (NewBucketIdSetting)
	{
		SetBucketId(SessionModificationHandle, NewBucketIdSetting->Data.GetString());
	}	
}

void FSessionsEOSGS::WriteUpdateSessionModificationHandle(EOS_HSessionModification& SessionModificationHandle, const FSessionSettingsUpdate& NewSettings)
{
	// We'll enable setting a custom host address if a value for it was set as a custom parameter
	if (const FCustomSessionSetting* HostAddressSetting = NewSettings.UpdatedCustomSettings.Find(EOSGS_HOST_ADDRESS_ATTRIBUTE_KEY))
	{
		FString HostAddressStr = HostAddressSetting->Data.GetString();
		if (!HostAddressStr.IsEmpty())
		{
			SetHostAddress(SessionModificationHandle, HostAddressStr);
		}
	}

	if (NewSettings.bAllowNewMembers.IsSet())
	{
		// We'll update this setting in the session modification step, and start or end the session accordingly to bock or allow join processes
		SetJoinInProgressAllowed(SessionModificationHandle, NewSettings.bAllowNewMembers.GetValue());

		// We'll also update invite permissions for the session
		SetInvitesAllowed(SessionModificationHandle, NewSettings.bAllowNewMembers.GetValue());
	}

	if (NewSettings.SchemaName.IsSet())
	{
		AddAttribute(SessionModificationHandle, EOSGS_SCHEMA_NAME_ATTRIBUTE_KEY, { FSchemaVariant(NewSettings.SchemaName.GetValue().ToString()), ESchemaAttributeVisibility::Public });
	}

	if (NewSettings.JoinPolicy.IsSet())
	{
		SetPermissionLevel(SessionModificationHandle, NewSettings.JoinPolicy.GetValue());
	}

	if (NewSettings.NumMaxConnections.IsSet())
	{
		SetMaxPlayers(SessionModificationHandle, NewSettings.NumMaxConnections.GetValue());
	}

	// BucketId has its own set method on the API
	const FCustomSessionSetting* NewBucketIdSetting = NewSettings.UpdatedCustomSettings.Find(EOSGS_BUCKET_ID_ATTRIBUTE_KEY);
	if (NewBucketIdSetting)
	{
		SetBucketId(SessionModificationHandle, NewBucketIdSetting->Data.GetString());
	}

	// Custom Settings

	for (const FSchemaAttributeId& Key : NewSettings.RemovedCustomSettings)
	{
		RemoveAttribute(SessionModificationHandle, Key);
	}

	for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : NewSettings.UpdatedCustomSettings)
	{
		AddAttribute(SessionModificationHandle, Entry.Key, Entry.Value);
	}
}

TFuture<TOnlineResult<FUpdateSessionSettings>> FSessionsEOSGS::UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params)
{
	// LAN Sessions
	TOnlineResult<FGetSessionByName> Result = GetSessionByName({ Params.SessionName });
	if (Result.IsOk() && Result.GetOkValue().Session->GetSessionInfo().bIsLANSession)
	{
		return FSessionsLAN::UpdateSessionSettingsImpl(Params);
	}

	// EOSGS Sessions

	TPromise<TOnlineResult<FUpdateSessionSettings>> Promise;
	TFuture<TOnlineResult<FUpdateSessionSettings>> Future = Promise.GetFuture();

	EOS_Sessions_UpdateSessionModificationOptions UpdateSessionModificationOptions = {};
	UpdateSessionModificationOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_UPDATESESSIONMODIFICATION_API_LATEST, 1);

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	UpdateSessionModificationOptions.SessionName = SessionNameUtf8.Get();

	EOS_HSessionModification SessionModificationHandle = nullptr;
	EOS_EResult ResultCode = EOS_Sessions_UpdateSessionModification(SessionsHandle, &UpdateSessionModificationOptions, &SessionModificationHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Error, "[FSessionsEOSGS::UpdateSession] EOS_Sessions_UpdateSessionModification failed with result [%ls]", *LexToString(ResultCode));

		Promise.EmplaceValue(Errors::FromEOSResult(ResultCode));

		return Future;
	}

	// After creating the session modification handle, we'll add all the updated data to it
	WriteUpdateSessionModificationHandle(SessionModificationHandle, Params.Mutations);

	// Whether we update joinability or not will depend on if bAllowNewMembers was set to a new value
	FUpdateSessionImplEOSGS::Params UpdateSessionImplEOSGSParams =
	{
		MakeShared<FSessionModificationHandleEOSGS>(SessionModificationHandle),
		Params.Mutations.bAllowNewMembers.IsSet() ?
			FUpdateSessionJoinabilityParams{ Params.SessionName, Params.Mutations.bAllowNewMembers.GetValue() } :
			TOptional<FUpdateSessionJoinabilityParams>() // If bAllowNewMembers is not set, the TOptional will be unset too
	};

	UpdateSessionImplEOSGS(MoveTemp(UpdateSessionImplEOSGSParams))
	.Next([this, Promise = MoveTemp(Promise), Params](TDefaultErrorResult<FUpdateSessionImplEOSGS>&& Result) mutable
	{
		if (Result.IsOk())
		{
			TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ Params.SessionName });
			if (GetMutableSessionByNameResult.IsOk())
			{
				TSharedRef<FSessionCommon> FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;

				// Now that the API Session update has processed successfully, we'll update our local session with the same data
				FSessionUpdate SessionUpdateData = BuildSessionUpdate(FoundSession, Params.Mutations);

				(*FoundSession) += SessionUpdateData;

				// We set the result and fire the event
				Promise.EmplaceValue(FUpdateSessionSettings::Result{ });

				FSessionUpdated SessionUpdatedEvent{ Params.SessionName, SessionUpdateData };
				SessionEvents.OnSessionUpdated.Broadcast(SessionUpdatedEvent);
			}
			else
			{
				Promise.EmplaceValue(GetMutableSessionByNameResult.GetErrorValue());
			}
		}
		else
		{
			Promise.EmplaceValue(Result.GetErrorValue());
		}
	});

	return Future;
}

TFuture<TDefaultErrorResult<FUpdateSessionImplEOSGS>> FSessionsEOSGS::UpdateSessionImplEOSGS(FUpdateSessionImplEOSGS::Params&& Params)
{
	EOS_Sessions_UpdateSessionOptions UpdateSessionOptions = {};
	UpdateSessionOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_UPDATESESSION_API_LATEST, 1);

	UpdateSessionOptions.SessionModificationHandle = Params.SessionModificationHandle->ModificationHandle;

	TPromise<TDefaultErrorResult<FUpdateSessionImplEOSGS>> Promise;
	TFuture<TDefaultErrorResult<FUpdateSessionImplEOSGS>> Future = Promise.GetFuture();

	EOS_Async(EOS_Sessions_UpdateSession, SessionsHandle, UpdateSessionOptions,
	[this, Promise = MoveTemp(Promise), Params = MoveTemp(Params)](const EOS_Sessions_UpdateSessionCallbackInfo* Result) mutable
	{
		// If we only change bAllowNewMembers, the session update will yield an EOS_NoChange result, but we still need to continue to the next step
		if (Result->ResultCode != EOS_EResult::EOS_Success && Result->ResultCode != EOS_EResult::EOS_Sessions_OutOfSync && Result->ResultCode != EOS_EResult::EOS_NoChange)
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_UpdateSession failed with result [%ls]", *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		// After the successful general update, if indicated, we'll update the joinability
		if (Params.UpdateJoinabilitySettings.IsSet())
		{
			UpdateSessionJoinabilityImpl(MoveTemp(Params.UpdateJoinabilitySettings.GetValue()))
				.Next([this, NewSessionId = UTF8_TO_TCHAR(Result->SessionId), Promise = MoveTemp(Promise)](TDefaultErrorResult<FUpdateSessionJoinabilityImpl>&& Result) mutable
			{
				if (Result.IsOk())
				{
					Promise.EmplaceValue(FUpdateSessionImplEOSGS::Result{ NewSessionId });
				}
				else
				{
					Promise.EmplaceValue(Result.GetErrorValue());
				}
			});
		}
		else
		{
			Promise.EmplaceValue(FUpdateSessionImplEOSGS::Result{ FString(Result->SessionId) });
		}
	});

	return Future;
}

TFuture<TDefaultErrorResult<FUpdateSessionJoinabilityImpl>> FSessionsEOSGS::UpdateSessionJoinabilityImpl(FUpdateSessionJoinabilityImpl::Params&& Params)
{
	TPromise<TDefaultErrorResult<FUpdateSessionJoinabilityImpl>> Promise;
	TFuture<TDefaultErrorResult<FUpdateSessionJoinabilityImpl>> Future = Promise.GetFuture();

	// We get the active session handle with the session name
	EOS_Sessions_CopyActiveSessionHandleOptions CopyActiveSessionHandleOptions = {};
	CopyActiveSessionHandleOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_COPYACTIVESESSIONHANDLE_API_LATEST, 1);

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	CopyActiveSessionHandleOptions.SessionName = SessionNameUtf8.Get();

	EOS_HActiveSession ActiveSessionHandle;
	EOS_EResult CopyActiveSessionHandleResult = EOS_Sessions_CopyActiveSessionHandle(SessionsHandle, &CopyActiveSessionHandleOptions, &ActiveSessionHandle);
	if (CopyActiveSessionHandleResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_CopyActiveSessionHandle failed with result [%ls]", *LexToString(CopyActiveSessionHandleResult));
	}

	// We get the active session info with the handle
	EOS_ActiveSession_CopyInfoOptions CopyInfoOptions = {};
	CopyInfoOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_ACTIVESESSION_COPYINFO_API_LATEST, 1);

	EOS_ActiveSession_Info* ActiveSessionInfo = nullptr;
	EOS_EResult CopyInfoResult = EOS_ActiveSession_CopyInfo(ActiveSessionHandle, &CopyInfoOptions, &ActiveSessionInfo);
	if (CopyInfoResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "EOS_ActiveSession_CopyInfo failed with result [%ls]", *LexToString(CopyInfoResult));
	}

	// If not, we start or end the session to make it joinable or not (as we set JIP to false at creation time)
	if (Params.bAllowNewMembers)
	{
		// We check state. If bAllowNewMembers is true and session has not started, there's no need to do anything
		if (ActiveSessionInfo->State == EOS_EOnlineSessionState::EOS_OSS_InProgress)
		{
			EOS_Sessions_EndSessionOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ENDSESSION_API_LATEST, 1);

			Options.SessionName = SessionNameUtf8.Get();

			EOS_Async(EOS_Sessions_EndSession, SessionsHandle, Options,
				[this, Promise = MoveTemp(Promise)](const EOS_Sessions_EndSessionCallbackInfo* Result) mutable
			{
				if (Result->ResultCode != EOS_EResult::EOS_Success && Result->ResultCode != EOS_EResult::EOS_Sessions_OutOfSync)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_EndSession failed with result [%ls]", *LexToString(Result->ResultCode));
					Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
					return;
				}

				Promise.EmplaceValue(FUpdateSessionJoinabilityImpl::Result{ });
			});
		}
		else
		{
			Promise.EmplaceValue(FUpdateSessionJoinabilityImpl::Result{ });
		}
	}
	else
	{
		// We check state. If bAllowNewMembers is false and the session has started, there's no need to do anything
		if (ActiveSessionInfo->State == EOS_EOnlineSessionState::EOS_OSS_Pending || ActiveSessionInfo->State == EOS_EOnlineSessionState::EOS_OSS_Ended)
		{
			EOS_Sessions_StartSessionOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_STARTSESSION_API_LATEST, 1);

			Options.SessionName = SessionNameUtf8.Get();

			EOS_Async(EOS_Sessions_StartSession, SessionsHandle, Options,
				[this, Promise = MoveTemp(Promise)](const EOS_Sessions_StartSessionCallbackInfo* Result) mutable
			{
				if (Result->ResultCode != EOS_EResult::EOS_Success && Result->ResultCode != EOS_EResult::EOS_Sessions_OutOfSync)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_StartSession failed with result [%ls]", *LexToString(Result->ResultCode));
					Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
					return;
				}

				Promise.EmplaceValue(FUpdateSessionJoinabilityImpl::Result{ });
			});
		}
		else
		{
			Promise.EmplaceValue(FUpdateSessionJoinabilityImpl::Result{ });
		}
	}

	return Future;
}

TFuture<TDefaultErrorResult<FSendSingleSessionInviteImpl>> FSessionsEOSGS::SendSingleSessionInviteImpl(FSendSingleSessionInviteImpl::Params&& Params)
{
	TPromise<TDefaultErrorResult<FSendSingleSessionInviteImpl>> Promise;
	TFuture<TDefaultErrorResult<FSendSingleSessionInviteImpl>> Future = Promise.GetFuture();

	EOS_Sessions_SendInviteOptions SendInviteOptions = { };
	SendInviteOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_SENDINVITE_API_LATEST, 1);

	SendInviteOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	SendInviteOptions.SessionName = SessionNameUtf8.Get();

	SendInviteOptions.TargetUserId = GetProductUserIdChecked(Params.TargetAccountId);

	EOS_Async(EOS_Sessions_SendInvite, SessionsHandle, SendInviteOptions,
		[this, Promise = MoveTemp(Promise)](const EOS_Sessions_SendInviteCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_SendInvite failed with result [%ls]", *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
		}
		else
		{
			Promise.EmplaceValue(FSendSingleSessionInviteImpl::Result{ });
		}
	});

	return Future;
}

TFuture<TOnlineResult<FLeaveSession>> FSessionsEOSGS::LeaveSessionImpl(const FLeaveSession::Params& Params)
{
	// LAN Sessions
	TOnlineResult<FGetSessionByName> Result = GetSessionByName({ Params.SessionName });
	if (Result.IsOk() && Result.GetOkValue().Session->GetSessionInfo().bIsLANSession)
	{
		return FSessionsLAN::LeaveSessionImpl(Params);
	}

	// EOSGS Sessions

	TPromise<TOnlineResult<FLeaveSession>> Promise;
	TFuture<TOnlineResult<FLeaveSession>> Future = Promise.GetFuture();

	EOS_Sessions_DestroySessionOptions DestroySessionOptions = {};
	DestroySessionOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_DESTROYSESSION_API_LATEST, 1);

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	DestroySessionOptions.SessionName = SessionNameUtf8.Get();

	EOS_Async(EOS_Sessions_DestroySession, SessionsHandle, DestroySessionOptions,
		[this, Promise = MoveTemp(Promise), Params](const EOS_Sessions_DestroySessionCallbackInfo* Result) mutable
		{
			if (Result->ResultCode == EOS_EResult::EOS_Success || Result->ResultCode == EOS_EResult::EOS_NotFound)
			{
				TOnlineResult<FGetSessionByName> GetSessionByNameResult = GetSessionByName({ Params.SessionName });
				if (GetSessionByNameResult.IsOk())
				{
					TSharedRef<const ISession> FoundSession = GetSessionByNameResult.GetOkValue().Session;

					ClearSessionReferences(FoundSession->GetSessionId(), Params.SessionName, Params.LocalAccountId);
				}
				else if (GetSessionByNameResult.GetErrorValue() != Errors::NotFound())
				{
					Promise.EmplaceValue(GetSessionByNameResult.GetErrorValue());
					return; 
				}

				Promise.EmplaceValue(FLeaveSession::Result{ });

				FSessionLeft SessionLeftEvent;
				SessionLeftEvent.LocalAccountId = Params.LocalAccountId;
				SessionEvents.OnSessionLeft.Broadcast(SessionLeftEvent);	
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_DestroySession failed with result [%ls]", *LexToString(Result->ResultCode));
				Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
				return;
			}
		});

	return Future;
}

void FSessionsEOSGS::SetSessionSearchMaxResults(FSessionSearchHandleEOSGS& SessionSearchHandle, uint32 MaxResults)
{
	EOS_SessionSearch_SetMaxResultsOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_SETMAXSEARCHRESULTS_API_LATEST, 1);

	// We truncate the max results parameter up to the value of EOS_SESSIONS_MAX_SEARCH_RESULTS
	Options.MaxSearchResults = FMath::Clamp(MaxResults, 1, EOS_SESSIONS_MAX_SEARCH_RESULTS);

	EOS_EResult ResultCode = EOS_SessionSearch_SetMaxResults(SessionSearchHandle.SearchHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetSessionSearchMaxResults] EOS_SessionSearch_SetMaxResults failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetSessionSearchParameters(FSessionSearchHandleEOSGS& SessionSearchHandle, TArray<FFindSessionsSearchFilter> Filters)
{
	for (const FFindSessionsSearchFilter& Filter : Filters)
	{
		EOS_SessionSearch_SetParameterOptions Options = { };
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_SETPARAMETER_API_LATEST, 1);

		Options.ComparisonOp = ToServiceType(Filter.ComparisonOp);
		
		FSessionAttributeConverter<ESessionAttributeConversionType::ToService> Parameter(Filter.Key, Filter.Value);
		Options.Parameter = &Parameter.GetAttributeData();

		EOS_EResult ResultCode = EOS_SessionSearch_SetParameter(SessionSearchHandle.SearchHandle, &Options);
		if (ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetSessionSearchParameters] EOS_SessionSearch_SetParameter failed with result [%ls]", *LexToString(ResultCode));
		}
	}
}

void FSessionsEOSGS::SetSessionSearchSessionId(FSessionSearchHandleEOSGS& SessionSearchHandle, const FOnlineSessionId& SessionId)
{
	EOS_SessionSearch_SetSessionIdOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_SETSESSIONID_API_LATEST, 1);

	const auto SessionIdUtf8 = StringCast<UTF8CHAR>(*ToString(SessionId));
	Options.SessionId = (const char*)SessionIdUtf8.Get();

	EOS_EResult ResultCode = EOS_SessionSearch_SetSessionId(SessionSearchHandle.SearchHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetSessionSearchSessionId] EOS_SessionSearch_SetSessionId failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::SetSessionSearchTargetId(FSessionSearchHandleEOSGS& SessionSearchHandle, const FAccountId& TargetAccountId)
{
	EOS_SessionSearch_SetTargetUserIdOptions Options = { };
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_SETTARGETUSERID_API_LATEST, 1);
	Options.TargetUserId = GetProductUserIdChecked(TargetAccountId);

	EOS_EResult ResultCode = EOS_SessionSearch_SetTargetUserId(SessionSearchHandle.SearchHandle, &Options);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::SetSessionSearchTargetId] EOS_SessionSearch_SetTargetUserId failed with result [%ls]", *LexToString(ResultCode));
	}
}

void FSessionsEOSGS::WriteSessionSearchHandle(FSessionSearchHandleEOSGS& SessionSearchHandle, const FFindSessions::Params& Params)
{
	SetSessionSearchMaxResults(SessionSearchHandle, Params.MaxResults);

	if (!Params.Filters.IsEmpty())
	{
		SetSessionSearchParameters(SessionSearchHandle, Params.Filters);
	}

	if (Params.SessionId.IsSet())
	{
		SetSessionSearchSessionId(SessionSearchHandle, Params.SessionId.GetValue());
	}

	if (Params.TargetUser.IsSet())
	{
		SetSessionSearchTargetId(SessionSearchHandle, Params.TargetUser.GetValue());
	}
}

FOnlineSessionId FSessionsEOSGS::CreateSessionId(const FString& SessionId) const
{
	return FOnlineSessionIdRegistryEOSGS::GetRegistered(Services.GetServicesProvider()).BasicRegistry.FindOrAddHandle(SessionId);
}

FSessionInviteId FSessionsEOSGS::CreateSessionInviteId(const FString& SessionInviteId) const
{
	return FOnlineSessionInviteIdRegistryEOSGS::GetRegistered(Services.GetServicesProvider()).BasicRegistry.FindOrAddHandle(SessionInviteId);
}

TFuture<TOnlineResult<FFindSessions>> FSessionsEOSGS::FindSessionsImpl(const FFindSessions::Params& Params)
{
	// LAN Sessions
	if (Params.bFindLANSessions)
	{
		return FSessionsLAN::FindSessionsImpl(Params);
	}

	// EOSGS Sessions
	TPromise<TOnlineResult<FFindSessions>> Promise;
	TFuture<TOnlineResult<FFindSessions>> Future = Promise.GetFuture();

	// We start preparing the search
	EOS_Sessions_CreateSessionSearchOptions CreateSessionSearchOptions = { };
	CreateSessionSearchOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_CREATESESSIONSEARCH_API_LATEST, 1);
	CreateSessionSearchOptions.MaxSearchResults = FMath::Clamp(Params.MaxResults, 0, EOS_SESSIONS_MAX_SEARCH_RESULTS);

	EOS_HSessionSearch SearchHandle = nullptr;
	EOS_EResult ResultCode = EOS_Sessions_CreateSessionSearch(SessionsHandle, &CreateSessionSearchOptions, &SearchHandle);
	if (ResultCode != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Error, "[FSessionsEOSGS::FindSessions] EOS_Sessions_CreateSessionSearch failed with result [%ls]", *LexToString(ResultCode));

		Promise.EmplaceValue(Errors::FromEOSResult(ResultCode));
		return Future;
	}

	const TSharedRef<FSessionSearchHandleEOSGS>& CurrentSessionSearchHandleEOSGS = CurrentSessionSearchHandleEOSGSUserMap.Emplace(Params.LocalAccountId, MakeShared<FSessionSearchHandleEOSGS>(SearchHandle));

	// Before we start the search, we reset the cache and save the promise
	SearchResultsUserMap.FindOrAdd(Params.LocalAccountId).Reset();
	CurrentSessionSearchPromisesUserMap.Emplace(Params.LocalAccountId, MoveTemp(Promise));

	// We write the search attributes
	WriteSessionSearchHandle(*CurrentSessionSearchHandleEOSGS, Params);

	EOS_SessionSearch_FindOptions FindOptions = { };
	FindOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_FIND_API_LATEST, 2);
	FindOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

	EOS_Async(EOS_SessionSearch_Find, CurrentSessionSearchHandleEOSGS->SearchHandle, FindOptions,
		[this, Params](const EOS_SessionSearch_FindCallbackInfo* FindCallbackInfoResult) mutable
		{
			TPromise<TOnlineResult<FFindSessions>>& Promise = CurrentSessionSearchPromisesUserMap.FindChecked(Params.LocalAccountId);

			if (FindCallbackInfoResult->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_SessionSearch_Find failed with result [%ls]", *LexToString(FindCallbackInfoResult->ResultCode));
				Promise.EmplaceValue(Errors::FromEOSResult(FindCallbackInfoResult->ResultCode));
				CurrentSessionSearchPromisesUserMap.Remove(Params.LocalAccountId);
				CurrentSessionSearchHandleEOSGSUserMap.Remove(Params.LocalAccountId);
				return;
			}

			const TSharedRef<FSessionSearchHandleEOSGS>& CurrentSessionSearchHandleEOSGS = CurrentSessionSearchHandleEOSGSUserMap.FindChecked(Params.LocalAccountId);

			// For a successful session, we'll get the search results
			EOS_SessionSearch_GetSearchResultCountOptions GetSearchResultCountOptions = { };
			GetSearchResultCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_GETSEARCHRESULTCOUNT_API_LATEST, 1);
			int32 NumSearchResults = EOS_SessionSearch_GetSearchResultCount(CurrentSessionSearchHandleEOSGS->SearchHandle, &GetSearchResultCountOptions);

			EOS_SessionSearch_CopySearchResultByIndexOptions CopySearchResultByIndexOptions = { };
			CopySearchResultByIndexOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONSEARCH_COPYSEARCHRESULTBYINDEX_API_LATEST, 1);

			TArray<TFuture<TDefaultErrorResult<FBuildSessionFromDetailsHandle>>> PendingSessionsBuilt;

			for (int32 Index = 0; Index < NumSearchResults; Index++)
			{
				EOS_HSessionDetails SessionDetailsHandle = nullptr;

				CopySearchResultByIndexOptions.SessionIndex = Index;
				EOS_EResult CopySearchResultByIndexResult = EOS_SessionSearch_CopySearchResultByIndex(CurrentSessionSearchHandleEOSGS->SearchHandle, &CopySearchResultByIndexOptions, &SessionDetailsHandle);
				if (CopySearchResultByIndexResult == EOS_EResult::EOS_Success)
				{
					TSharedRef<TPromise<TDefaultErrorResult<FBuildSessionFromDetailsHandle>>> BuildSessionPromise = MakeShared<TPromise<TDefaultErrorResult<FBuildSessionFromDetailsHandle>>>();
					BuildSessionFromDetailsHandle({ Params.LocalAccountId, MakeShared<FSessionDetailsHandleEOSGS>(SessionDetailsHandle) })
						.OnComplete([this, BuildSessionPromise](const TOnlineResult<FBuildSessionFromDetailsHandle>& Result) mutable
					{
						if (Result.IsOk())
						{
							AddSearchResult(Result.GetOkValue().Session, Result.GetOkValue().LocalAccountId);

							BuildSessionPromise->EmplaceValue(Result.GetOkValue());
						}
						else
						{
							BuildSessionPromise->EmplaceValue(Result.GetErrorValue());
						}
					});
			
					PendingSessionsBuilt.Add(BuildSessionPromise->GetFuture());
				}
				else
				{
					UE_LOGF(LogOnlineServices, Error, "[FSessionsEOSGS::FindSessions] EOS_SessionSearch_CopySearchResultByIndex failed for index [%d] with result [%ls]", Index, *LexToString(CopySearchResultByIndexResult));

					Promise.EmplaceValue(Errors::FromEOSResult(CopySearchResultByIndexResult));
					return;
				}
			}

			WhenAll(MoveTemp(PendingSessionsBuilt))
				.Next([this, Params](TArray<TDefaultErrorResult<FBuildSessionFromDetailsHandle>>&& Results) mutable
			{
				TPromise<TOnlineResult<FFindSessions>>& Promise = CurrentSessionSearchPromisesUserMap.FindChecked(Params.LocalAccountId);

				for (TDefaultErrorResult<FBuildSessionFromDetailsHandle>& Result : Results)
				{
					if (Result.IsError())
					{
						// Store first encountered error to return as result.
						Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
						CurrentSessionSearchPromisesUserMap.Remove(Params.LocalAccountId);
						CurrentSessionSearchHandleEOSGSUserMap.Remove(Params.LocalAccountId);
						return;
					}
				}

				TArray<FOnlineSessionId>& SearchResults = SearchResultsUserMap.FindChecked(Params.LocalAccountId);

				Promise.EmplaceValue(FFindSessions::Result{ SearchResults });
				CurrentSessionSearchPromisesUserMap.Remove(Params.LocalAccountId);
				CurrentSessionSearchHandleEOSGSUserMap.Remove(Params.LocalAccountId);
			});
		});

	return Future;
}

TFuture<TOnlineResult<FJoinSession>> FSessionsEOSGS::JoinSessionImpl(const FJoinSession::Params& Params)
{
	TPromise<TOnlineResult<FJoinSession>> Promise;
	TFuture<TOnlineResult<FJoinSession>> Future = Promise.GetFuture();

	TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
	if (GetSessionByIdResult.IsError())
	{
		// If no result is found, the id might be expired, which we should notify
		if (FOnlineSessionIdRegistryEOSGS::GetRegistered(Services.GetServicesProvider()).IsSessionIdExpired(Params.SessionId))
		{
			UE_LOGF(LogOnlineServices, Warning, "[%ls] SessionId parameter [%ls] is expired. Please call FindSessions to get an updated list of available sessions ", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
		}

		Promise.EmplaceValue(GetSessionByIdResult.GetErrorValue());
		return Future;
	}

	const TSharedRef<const ISession>& FoundSession = GetSessionByIdResult.GetOkValue().Session;

	// LAN Sessions
	if (FoundSession->GetSessionInfo().bIsLANSession)
	{
		return FSessionsLAN::JoinSessionImpl(Params);
	}

	// EOSGS Sessions

	// We start setup for the API call
	EOS_Sessions_JoinSessionOptions JoinSessionOptions = { };
	JoinSessionOptions.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_JOINSESSION_API_LATEST, 2);

	JoinSessionOptions.bPresenceEnabled = Params.bPresenceEnabled;

	JoinSessionOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	JoinSessionOptions.SessionName = SessionNameUtf8.Get();

	const FSessionEOSGS& SessionEOSGS = FSessionEOSGS::Cast(*FoundSession);
	JoinSessionOptions.SessionHandle = SessionEOSGS.SessionDetailsHandle->SessionDetailsHandle;

	EOS_Async(EOS_Sessions_JoinSession, SessionsHandle, JoinSessionOptions, 
		[this, Promise = MoveTemp(Promise), Params](const EOS_Sessions_JoinSessionCallbackInfo* Result) mutable
		{
			if (Result->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_JoinSession failed with result [%ls]", *LexToString(Result->ResultCode));
				Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
				return;
			}
		
			TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
			if (GetSessionByIdResult.IsError())
			{
				// If no result is found, the id might be expired, which we should notify
				if (FOnlineSessionIdRegistryEOSGS::GetRegistered(Services.GetServicesProvider()).IsSessionIdExpired(Params.SessionId))
				{
					UE_LOGF(LogOnlineServices, Warning, "[%ls] SessionId parameter [%ls] is expired. Please call FindSessions to get an updated list of available sessions ", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
				}

				Promise.EmplaceValue(MoveTemp(GetSessionByIdResult.GetErrorValue()));
				return;
			}

			const TSharedRef<const ISession>& FoundSession = GetSessionByIdResult.GetOkValue().Session;

			AddSessionReferences(FoundSession->GetSessionId(), Params.SessionName, Params.LocalAccountId, Params.bPresenceEnabled);

			// After successfully joining a session, we'll remove all related invites if any are found
			ClearSessionInvitesForSession(Params.LocalAccountId, FoundSession->GetSessionId());

			Promise.EmplaceValue(FJoinSession::Result{ });

			FSessionJoined Event = { { Params.LocalAccountId }, FoundSession->GetSessionId() };

			SessionEvents.OnSessionJoined.Broadcast(Event);

			// A successful join allows the client to server travel, after which AddSessionMember will be called by the engine
		});

	return Future;
}

TOptional<FOnlineError> FSessionsEOSGS::CheckState(const FJoinSession::Params& Params) const
{
	if (TOptional<FOnlineError> BaseCheck = Super::CheckState(Params))
	{
		return BaseCheck;
	}

	TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
	if (GetSessionByIdResult.IsOk())
	{
		const TSharedRef<const ISession>& FoundSession = GetSessionByIdResult.GetOkValue().Session;

		// We check that the passed session has a valid details handle
		const FSessionEOSGS& SessionEOSGS = FSessionEOSGS::Cast(*FoundSession);
		if (!FoundSession->GetSessionInfo().bIsLANSession && !SessionEOSGS.SessionDetailsHandle.IsValid())
		{
			UE_LOGF(LogOnlineServices, Warning, "[%ls] Could not join session with invalid session details handle in session with id [%ls]", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));

			return TOptional<FOnlineError>(Errors::InvalidState());
		}
	}
	else
	{
		return TOptional<FOnlineError>(GetSessionByIdResult.GetErrorValue());
	}

	return TOptional<FOnlineError>();
}

TOnlineAsyncOpHandle<FBuildSessionFromDetailsHandle> FSessionsEOSGS::BuildSessionFromInvite(const FAccountId& LocalAccountId, const FString& InInviteId)
{
	EOS_Sessions_CopySessionHandleByInviteIdOptions CopySessionHandleByInviteIdOptions = { };
	CopySessionHandleByInviteIdOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_COPYSESSIONHANDLEBYINVITEID_API_LATEST, 1);

	const FTCHARToUTF8 InviteIdUtf8(*InInviteId);
	CopySessionHandleByInviteIdOptions.InviteId = InviteIdUtf8.Get();

	EOS_HSessionDetails SessionDetailsHandle;
	EOS_EResult CopySessionHandleByInviteIdResult = EOS_Sessions_CopySessionHandleByInviteId(SessionsHandle, &CopySessionHandleByInviteIdOptions, &SessionDetailsHandle);
	if (CopySessionHandleByInviteIdResult == EOS_EResult::EOS_Success)
	{
		return BuildSessionFromDetailsHandle({ LocalAccountId, MakeShared<FSessionDetailsHandleEOSGS>(SessionDetailsHandle) });
	}
	else
	{
		UE_LOGF(LogOnlineServices, Error, "[FSessionsEOSGS::BuildSessionFromInvite] EOS_Sessions_CopySessionHandleByInviteId failed with result [%ls]", *LexToString(CopySessionHandleByInviteIdResult));

		TOnlineAsyncOpPtr<FBuildSessionFromDetailsHandle> Operation;
		Operation->SetError(Errors::FromEOSResult(CopySessionHandleByInviteIdResult));
		return Operation->GetHandle();
	}
}

TOnlineAsyncOpHandle<FBuildSessionFromDetailsHandle> FSessionsEOSGS::BuildSessionFromUIEvent(const FAccountId& LocalAccountId, const EOS_UI_EventId& UIEventId)
{
	EOS_Sessions_CopySessionHandleByUiEventIdOptions CopySessionHandleByUiEventIdOptions = { };
	CopySessionHandleByUiEventIdOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_COPYSESSIONHANDLEBYUIEVENTID_API_LATEST, 1);

	CopySessionHandleByUiEventIdOptions.UiEventId = UIEventId;

	EOS_HSessionDetails SessionDetailsHandle;
	EOS_EResult CopySessionHandleByUiEventIdResult = EOS_Sessions_CopySessionHandleByUiEventId(SessionsHandle, &CopySessionHandleByUiEventIdOptions, &SessionDetailsHandle);
	if (CopySessionHandleByUiEventIdResult == EOS_EResult::EOS_Success)
	{
		return BuildSessionFromDetailsHandle({ LocalAccountId, MakeShared<FSessionDetailsHandleEOSGS>(SessionDetailsHandle) });
	}
	else
	{
		UE_LOGF(LogOnlineServices, Error, "[FSessionsEOSGS::BuildSessionFromUIEvent] EOS_Sessions_CopySessionHandleByUiEventId failed with result [%ls]", *LexToString(CopySessionHandleByUiEventIdResult));

		TOnlineAsyncOpPtr<FBuildSessionFromDetailsHandle> Operation;
		Operation->SetError(Errors::FromEOSResult(CopySessionHandleByUiEventIdResult));
		return Operation->GetHandle();
	}
}

TResult<TArray<EOS_ProductUserId>, FOnlineError> GetProductUserIdsFromEOSGSSession(const EOS_HSessionDetails& SessionDetailsHandle)
{
	TArray<EOS_ProductUserId> Result;

	// This code will remain commented until the related functionality is made available by EOSSDK

	/*EOS_SessionDetails_CopyInfoOptions CopyInfoOptions = {};
	CopyInfoOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONDETAILS_COPYINFO_API_LATEST, 1);

	EOS_SessionDetails_Info* SessionDetailsInfo = nullptr;
	EOS_EResult CopyInfoResult = EOS_SessionDetails_CopyInfo(SessionDetailsHandle, &CopyInfoOptions, &SessionDetailsInfo);
	if (CopyInfoResult == EOS_EResult::EOS_Success)
	{
		// TODO: When available, we'll retrieve the Session's Owner Id here.
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsEOSGS::BuildSessionFromDetailsHandle] EOS_SessionDetails_CopyInfo failed with result [%ls]", *LexToString(CopyInfoResult));

		TResult<TArray<EOS_ProductUserId>, FOnlineError>(Errors::FromEOSResult(CopyInfoResult));
	}*/

	return TResult<TArray<EOS_ProductUserId>, FOnlineError>(MoveTemp(Result));
}

TOnlineAsyncOpHandle<FBuildSessionFromDetailsHandle> FSessionsEOSGS::BuildSessionFromDetailsHandle(FBuildSessionFromDetailsHandle::Params&& Params)
{
	TOnlineAsyncOpRef<FBuildSessionFromDetailsHandle> Op = GetOp<FBuildSessionFromDetailsHandle>(MoveTemp(Params));
	const FBuildSessionFromDetailsHandle::Params& OpParams = Op->GetParams();

	// The first step in the process will be retrieving all ids in the session that need resolution
	TResult<TArray<EOS_ProductUserId>, FOnlineError> GetIdsResult = GetProductUserIdsFromEOSGSSession(Params.SessionDetailsHandleEOSGS->SessionDetailsHandle);

	if (GetIdsResult.IsOk())
	{
		Services.Get<IEpicProductUserIdResolver>()->ResolveAccountIds(Params.LocalAccountId, GetIdsResult.GetOkValue())
			.Next([this, WeakOp = Op->AsWeak(), Params = MoveTemp(Params)](TArray<FAccountId> ResolvedAccountIds) mutable
		{
			if (TOnlineAsyncOpPtr<FBuildSessionFromDetailsHandle> StrongOp = WeakOp.Pin())
			{
				// After all the ids are resolved, we can build the session safely
				StrongOp->SetResult(FBuildSessionFromDetailsHandle::Result{ StrongOp->GetParams().LocalAccountId, MakeShared<FSessionEOSGS>(*this, Params.SessionDetailsHandleEOSGS) });
			}
		});
	}
	else
	{
		Op->SetError(MoveTemp(GetIdsResult.GetErrorValue()));
	}

	return Op->GetHandle();
}

TFuture<TOnlineResult<FSendSessionInvite>> FSessionsEOSGS::SendSessionInviteImpl(const FSendSessionInvite::Params& Params)
{
	TPromise<TOnlineResult<FSendSessionInvite>> Promise;
	TFuture<TOnlineResult<FSendSessionInvite>> Future = Promise.GetFuture();

	TArray<TFuture<TDefaultErrorResult<FSendSingleSessionInviteImpl>>> PendingSessionInvites;
	for (const FAccountId& TargetAccountId : Params.TargetUsers)
	{
		FSendSingleSessionInviteImpl::Params FSendSingleSessionInviteParams;
		FSendSingleSessionInviteParams.LocalAccountId = Params.LocalAccountId;
		FSendSingleSessionInviteParams.SessionName = Params.SessionName;
		FSendSingleSessionInviteParams.TargetAccountId = TargetAccountId;

		TPromise<TDefaultErrorResult<FSendSingleSessionInviteImpl>> SessionInvitePromise;
		PendingSessionInvites.Emplace(SessionInvitePromise.GetFuture());

		SendSingleSessionInviteImpl(MoveTemp(FSendSingleSessionInviteParams))
			.Next([SessionInvitePromise = MoveTemp(SessionInvitePromise)](TDefaultErrorResult<FSendSingleSessionInviteImpl>&& Result) mutable
				{
					if (Result.IsOk())
					{
						SessionInvitePromise.EmplaceValue(MoveTemp(Result.GetOkValue()));
					}
					else
					{
						SessionInvitePromise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
					}
				});
	}

	WhenAll(MoveTemp(PendingSessionInvites))
		.Next([Promise = MoveTemp(Promise)](TArray<TDefaultErrorResult<FSendSingleSessionInviteImpl>>&& Results) mutable
	{
		for (TDefaultErrorResult<FSendSingleSessionInviteImpl>& Result : Results)
		{
			if (Result.IsError())
			{
				Promise.EmplaceValue(MoveTemp(Result.GetErrorValue()));
				return;
			}
		}

		Promise.EmplaceValue(FSendSessionInvite::Result{ });
	});

	return Future;
}

TFuture<TOnlineResult<FRejectSessionInvite>> FSessionsEOSGS::RejectSessionInviteImpl(const FRejectSessionInvite::Params& Params)
{
	TPromise<TOnlineResult<FRejectSessionInvite>> Promise;
	TFuture<TOnlineResult<FRejectSessionInvite>> Future = Promise.GetFuture();

	EOS_Sessions_RejectInviteOptions RejectInviteOptions = { };
	RejectInviteOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_REJECTINVITE_API_LATEST, 1);

	RejectInviteOptions.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

	const FString& InviteIdStr = FOnlineSessionInviteIdRegistryEOSGS::GetRegistered(Services.GetServicesProvider()).BasicRegistry.FindIdValue(Params.SessionInviteId);
	const FTCHARToUTF8 InviteIdUtf8(*InviteIdStr);
	RejectInviteOptions.InviteId = InviteIdUtf8.Get();

	EOS_Async(EOS_Sessions_RejectInvite, SessionsHandle, RejectInviteOptions, 
		[this, Promise = MoveTemp(Promise), Params](const EOS_Sessions_RejectInviteCallbackInfo* Result) mutable
		{
			if (Result->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_RejectInvite failed with result [%ls]", *LexToString(Result->ResultCode));
				Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
				return;
			}

			if (TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>* UserMap = SessionInvitesUserMap.Find(Params.LocalAccountId))
			{
				UserMap->Remove(Params.SessionInviteId);
			}

			Promise.EmplaceValue(FRejectSessionInvite::Result{ });
		});

	return Future;
}

TFuture<TOnlineResult<FAddSessionMember>> FSessionsEOSGS::AddSessionMemberImpl(const FAddSessionMember::Params& Params)
{
	// LAN Sessions
	TOnlineResult<FGetSessionByName> Result = GetSessionByName({ Params.SessionName });
	if (Result.IsOk() && Result.GetOkValue().Session->GetSessionInfo().bIsLANSession)
	{
		return FSessionsLAN::AddSessionMemberImpl(Params);
	}

	// EOSGS Sessions

	TPromise<TOnlineResult<FAddSessionMember>> Promise;
	TFuture<TOnlineResult<FAddSessionMember>> Future = Promise.GetFuture();

	EOS_Sessions_RegisterPlayersOptions Options = { };
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_REGISTERPLAYERS_API_LATEST, 3);

	Options.PlayersToRegisterCount = 1;

	TArray<EOS_ProductUserId> PlayersToRegister = { GetProductUserIdChecked(Params.LocalAccountId) };
	Options.PlayersToRegister = PlayersToRegister.GetData();

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	Options.SessionName = SessionNameUtf8.Get();

	EOS_Async(EOS_Sessions_RegisterPlayers, SessionsHandle, Options,
		[this, Promise = MoveTemp(Promise), Params](const EOS_Sessions_RegisterPlayersCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_RegisterPlayers failed with result [%ls]", *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		Promise.EmplaceValue(AddSessionMemberInternal(Params));
	});

	return Future;
}

TFuture<TOnlineResult<FRemoveSessionMember>> FSessionsEOSGS::RemoveSessionMemberImpl(const FRemoveSessionMember::Params& Params)
{
	// LAN Sessions
	TOnlineResult<FGetSessionByName> Result = GetSessionByName({ Params.SessionName });
	if (Result.IsOk() && Result.GetOkValue().Session->GetSessionInfo().bIsLANSession)
	{
		return FSessionsLAN::RemoveSessionMemberImpl(Params);
	}

	// EOSGS Sessions

	TPromise<TOnlineResult<FRemoveSessionMember>> Promise;
	TFuture<TOnlineResult<FRemoveSessionMember>> Future = Promise.GetFuture();

	
	EOS_Sessions_UnregisterPlayersOptions Options = { };
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_UNREGISTERPLAYERS_API_LATEST, 2);

	Options.PlayersToUnregisterCount = 1;

	TArray<EOS_ProductUserId> PlayersToUnregister{ GetProductUserIdChecked(Params.LocalAccountId) };
	Options.PlayersToUnregister = PlayersToUnregister.GetData();

	const FTCHARToUTF8 SessionNameUtf8(*Params.SessionName.ToString());
	Options.SessionName = SessionNameUtf8.Get();

	EOS_Async(EOS_Sessions_UnregisterPlayers, SessionsHandle, Options,
		[this, Promise = MoveTemp(Promise), Params](const EOS_Sessions_UnregisterPlayersCallbackInfo* Result) mutable
	{
		if (Result->ResultCode != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_Sessions_UnregisterPlayers failed with result [%ls]", *LexToString(Result->ResultCode));
			Promise.EmplaceValue(Errors::FromEOSResult(Result->ResultCode));
			return;
		}

		Promise.EmplaceValue(RemoveSessionMemberInternal(Params));
	});

	return Future;
}

/** FSessionsLAN */

void FSessionsEOSGS::AppendSessionToPacket(FNboSerializeToBuffer& Packet, const FSessionLAN& Session)
{
	using namespace NboSerializerLANSvc;
	using namespace NboSerializerEOSGSSvc;

	SerializeToBuffer(Packet, Session);
	SerializeToBuffer(Packet, Session.OwnerAccountId);
	SerializeToBuffer(Packet, Session.SessionInfo.SessionId);
	SerializeToBuffer(Packet, Session.SessionMembers);
}

void FSessionsEOSGS::ReadSessionFromPacket(FNboSerializeFromBuffer& Packet, FSessionLAN& Session)
{
	using namespace NboSerializerLANSvc;
	using namespace NboSerializerEOSGSSvc;

	SerializeFromBuffer(Packet, Session);
	SerializeFromBuffer(Packet, Session.OwnerAccountId);
	SerializeFromBuffer(Packet, Session.SessionInfo.SessionId);
	SerializeFromBuffer(Packet, Session.SessionMembers);
}

FAccountId FSessionsEOSGS::FindAccountId(const EOS_ProductUserId ProductUserId)
{
	return UE::Online::FindAccountId(Services.GetServicesProvider(), ProductUserId);
}

/* UE::Online */ }
