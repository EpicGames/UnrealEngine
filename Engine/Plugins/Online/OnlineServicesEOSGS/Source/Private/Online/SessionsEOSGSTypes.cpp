// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SessionsEOSGSTypes.h"
#include "Online/Sessions.h"

#include "EOSShared.h"

namespace UE::Online {

EOS_EOnlineSessionPermissionLevel ToServiceType(const ESessionJoinPolicy& Value)
{
	switch (Value)
	{
	case ESessionJoinPolicy::Public:		return EOS_EOnlineSessionPermissionLevel::EOS_OSPF_PublicAdvertised;
	case ESessionJoinPolicy::FriendsOnly:	return EOS_EOnlineSessionPermissionLevel::EOS_OSPF_JoinViaPresence;
	case ESessionJoinPolicy::InviteOnly:	return EOS_EOnlineSessionPermissionLevel::EOS_OSPF_InviteOnly;
	}

	checkNoEntry();
	return EOS_EOnlineSessionPermissionLevel::EOS_OSPF_InviteOnly;
}

ESessionJoinPolicy FromServiceType(const EOS_EOnlineSessionPermissionLevel& Value)
{
	switch (Value)
	{
	case EOS_EOnlineSessionPermissionLevel::EOS_OSPF_PublicAdvertised:	return ESessionJoinPolicy::Public;
	case EOS_EOnlineSessionPermissionLevel::EOS_OSPF_JoinViaPresence:	return ESessionJoinPolicy::FriendsOnly;
	case EOS_EOnlineSessionPermissionLevel::EOS_OSPF_InviteOnly:		return ESessionJoinPolicy::InviteOnly;
	}

	checkNoEntry();
	return ESessionJoinPolicy::InviteOnly;
}

EOS_ESessionAttributeAdvertisementType ToServiceType(const ESchemaAttributeVisibility& Value)
{
	switch (Value)
	{
	case ESchemaAttributeVisibility::Private: return EOS_ESessionAttributeAdvertisementType::EOS_SAAT_DontAdvertise;
	case ESchemaAttributeVisibility::Public: return EOS_ESessionAttributeAdvertisementType::EOS_SAAT_Advertise;
	}

	checkNoEntry();
	return EOS_ESessionAttributeAdvertisementType::EOS_SAAT_DontAdvertise;
}

ESchemaAttributeVisibility FromServiceType(const EOS_ESessionAttributeAdvertisementType& Value)
{
	switch (Value)
	{
	case EOS_ESessionAttributeAdvertisementType::EOS_SAAT_DontAdvertise: return ESchemaAttributeVisibility::Private;
	case EOS_ESessionAttributeAdvertisementType::EOS_SAAT_Advertise: return ESchemaAttributeVisibility::Public;
	}

	checkNoEntry();
	return ESchemaAttributeVisibility::Private;
}

EOS_EOnlineComparisonOp ToServiceType(const ESchemaAttributeComparisonOp& Value)
{
	switch (Value)
	{
	case ESchemaAttributeComparisonOp::Equals:			return EOS_EComparisonOp::EOS_CO_EQUAL;
	case ESchemaAttributeComparisonOp::NotEquals:			return EOS_EComparisonOp::EOS_CO_NOTEQUAL;
	case ESchemaAttributeComparisonOp::GreaterThan:		return EOS_EComparisonOp::EOS_CO_GREATERTHAN;
	case ESchemaAttributeComparisonOp::GreaterThanEquals:	return EOS_EComparisonOp::EOS_CO_GREATERTHANOREQUAL;
	case ESchemaAttributeComparisonOp::LessThan:			return EOS_EComparisonOp::EOS_CO_LESSTHAN;
	case ESchemaAttributeComparisonOp::LessThanEquals:	return EOS_EComparisonOp::EOS_CO_LESSTHANOREQUAL;
	case ESchemaAttributeComparisonOp::Near:				return EOS_EComparisonOp::EOS_CO_DISTANCE;
	case ESchemaAttributeComparisonOp::In:				return EOS_EComparisonOp::EOS_CO_ONEOF;
	case ESchemaAttributeComparisonOp::NotIn:				return EOS_EComparisonOp::EOS_CO_NOTANYOF;
	}

	checkNoEntry();
	return EOS_EComparisonOp::EOS_CO_EQUAL;
}

FSessionAttributeConverter<ESessionAttributeConversionType::ToService>::FSessionAttributeConverter(const FSchemaAttributeId& Key, const FSchemaVariant& Value)
	: FSessionAttributeConverter(TPair<FSchemaAttributeId, FSchemaVariant>(Key, Value))
{

}

FSessionAttributeConverter<ESessionAttributeConversionType::ToService>::FSessionAttributeConverter(const TPair<FSchemaAttributeId, FSchemaVariant>& InData)
	: KeyConverterStorage(*InData.Key.ToString())
{
	AttributeData.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_SESSIONS_ATTRIBUTEDATA_API_LATEST, 1);

	AttributeData.Key = KeyConverterStorage.Get();

	switch (InData.Value.VariantType)
	{
	case ESchemaAttributeType::Bool:
		AttributeData.ValueType = EOS_ESessionAttributeType::EOS_AT_BOOLEAN;
		AttributeData.Value.AsBool = InData.Value.GetBoolean();
		break;
	case ESchemaAttributeType::Double:
		AttributeData.ValueType = EOS_ESessionAttributeType::EOS_AT_DOUBLE;
		AttributeData.Value.AsDouble = InData.Value.GetDouble();
		break;
	case ESchemaAttributeType::Int64:
		AttributeData.ValueType = EOS_ESessionAttributeType::EOS_AT_INT64;
		AttributeData.Value.AsInt64 = InData.Value.GetInt64();
		break;
	case ESchemaAttributeType::String:
		AttributeData.ValueType = EOS_ESessionAttributeType::EOS_AT_STRING;
		ValueConverterStorage.Emplace(*InData.Value.GetString());
		AttributeData.Value.AsUtf8 = ValueConverterStorage->Get();
		break;
	}
}

FSessionAttributeConverter<ESessionAttributeConversionType::FromService>::FSessionAttributeConverter(const EOS_Sessions_AttributeData& InData)
{
	FSchemaAttributeId AttributeId = FSchemaAttributeId(UTF8_TO_TCHAR(InData.Key));
	FSchemaVariant VariantData;

	switch (InData.ValueType)
	{
	case EOS_ESessionAttributeType::EOS_AT_BOOLEAN:
		VariantData.Set(InData.Value.AsBool != 0);
		break;
	case EOS_ESessionAttributeType::EOS_AT_INT64:
		VariantData.Set(static_cast<int64>(InData.Value.AsInt64));
		break;
	case EOS_ESessionAttributeType::EOS_AT_DOUBLE:
		VariantData.Set(InData.Value.AsDouble);
		break;
	case EOS_ESessionAttributeType::EOS_AT_STRING:
		VariantData.Set(UTF8_TO_TCHAR(InData.Value.AsUtf8));
		break;
	default:
		checkNoEntry();
		break;
	}

	AttributeData = TPair<FSchemaAttributeId, FSchemaVariant>(MoveTemp(AttributeId), MoveTemp(VariantData));
}

/* UE::Online */ }
