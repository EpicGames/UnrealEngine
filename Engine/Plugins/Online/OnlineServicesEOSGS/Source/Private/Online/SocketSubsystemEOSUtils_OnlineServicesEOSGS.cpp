// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ENGINE
#include "SocketSubsystemEOSUtils_OnlineServicesEOSGS.h"

#include "Online/Auth.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/Lobbies.h"
#include "Online/OnlineExecHandler.h"
#include "Online/OnlineServicesEOSGS.h"
#include "Online/OnlineServicesLog.h"
#include "Online/OnlineUtils.h"

namespace UE::Online {

FSocketSubsystemEOSUtils_OnlineServicesEOS::FSocketSubsystemEOSUtils_OnlineServicesEOS(FOnlineServicesEOSGS& InServicesEOSGS)
	: ServicesEOSGS(InServicesEOSGS)
{
}

FSocketSubsystemEOSUtils_OnlineServicesEOS::~FSocketSubsystemEOSUtils_OnlineServicesEOS()
{
}

#if WITH_EOS_P2P
EOS_ProductUserId FSocketSubsystemEOSUtils_OnlineServicesEOS::GetLocalUserId()
{
	EOS_ProductUserId Result = nullptr;

	IAuthPtr AuthEOS = ServicesEOSGS.GetAuthInterface();
	check(AuthEOS);

	FAuthGetLocalOnlineUserByPlatformUserId::Params AuthParams;
	AuthParams.PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(0);
	TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> AuthResult = AuthEOS->GetLocalOnlineUserByPlatformUserId(MoveTemp(AuthParams));
	if (AuthResult.IsOk())
	{
		UE::Online::FAuthGetLocalOnlineUserByPlatformUserId::Result OkValue = AuthResult.GetOkValue();

		Result = GetProductUserIdChecked(OkValue.AccountInfo->AccountId);
	}
	else
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FSocketSubsystemEOSUtils_OnlineServicesEOS::GetLocalUserId] Unable to get account for platform user id [%ls]. Error=[%ls]", *ToLogString(AuthParams.PlatformUserId), *AuthResult.GetErrorValue().GetLogString(true));
	}

	return Result;
}
#endif // WITH_EOS_P2P

bool FSocketSubsystemEOSUtils_OnlineServicesEOS::IsLoggedIn()
{
	IAuthPtr Auth = ServicesEOSGS.GetAuthInterface();
	check(Auth);

	return Auth->IsLoggedIn(FPlatformMisc::GetPlatformUserForUserIndex(0));
}

FString FSocketSubsystemEOSUtils_OnlineServicesEOS::GetSessionId()
{
	FString Result;

	IAuthPtr AuthEOS = ServicesEOSGS.GetAuthInterface();
	check(AuthEOS);

	FAuthGetLocalOnlineUserByPlatformUserId::Params AuthParams;
	AuthParams.PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(0);
	TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> AuthResult = AuthEOS->GetLocalOnlineUserByPlatformUserId(MoveTemp(AuthParams));
	if (AuthResult.IsOk())
	{
		FAuthGetLocalOnlineUserByPlatformUserId::Result* AuthOkValue = AuthResult.TryGetOkValue();

		ILobbiesPtr LobbiesEOS = ServicesEOSGS.GetLobbiesInterface();
		check(LobbiesEOS);

		FGetJoinedLobbies::Params LobbiesParams;
		LobbiesParams.LocalAccountId = AuthOkValue->AccountInfo->AccountId;
		TOnlineResult<FGetJoinedLobbies> LobbiesResult = LobbiesEOS->GetJoinedLobbies(MoveTemp(LobbiesParams));
		if (LobbiesResult.IsOk())
		{
			FGetJoinedLobbies::Result* LobbiesOkValue = LobbiesResult.TryGetOkValue();

			// TODO: Pending support in Lobbies interface
			/*for (TSharedRef<FLobby> Lobby : LobbiesOkValue->Lobbies)
			{
					if (Lobby->SessionName == NAME_GameSession)
					Result = LobbiesEOS->GetLobbyIdString(Lobby);
			}*/
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[FSocketSubsystemEOSUtils_OnlineServicesEOS::GetSessionId] Unable to get joined lobbies for local user id [%ls]. Error=[%ls]", *ToLogString(LobbiesParams.LocalAccountId), *AuthResult.GetErrorValue().GetLogString(true));
		}
	}
	else
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FSocketSubsystemEOSUtils_OnlineServicesEOS::GetSessionId] Unable to get account for platform user id [%ls]. Error=[%ls]", *ToLogString(AuthParams.PlatformUserId), *AuthResult.GetErrorValue().GetLogString(true));
	}

	return Result;
}

FName FSocketSubsystemEOSUtils_OnlineServicesEOS::GetSubsystemInstanceName()
{
	return ServicesEOSGS.GetInstanceName();
}

/* UE::Online */}

#endif // WITH_ENGINE
