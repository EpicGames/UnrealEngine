// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ENGINE

#include "SocketSubsystemEOSUtils.h"

namespace UE::Online {

class FOnlineServicesEOSGS;

class FSocketSubsystemEOSUtils_OnlineServicesEOS : public ISocketSubsystemEOSUtils
{
public:
	FSocketSubsystemEOSUtils_OnlineServicesEOS(FOnlineServicesEOSGS& InServicesEOS);
	virtual ~FSocketSubsystemEOSUtils_OnlineServicesEOS();

#if WITH_EOS_P2P
	virtual EOS_ProductUserId GetLocalUserId() override;
#endif
	virtual FString GetSessionId() override;
	virtual FName GetSubsystemInstanceName() override;
	virtual bool IsLoggedIn() override;

private:
	FOnlineServicesEOSGS& ServicesEOSGS;
};

/* UE::Online */}

#endif // WITH_ENGINE
