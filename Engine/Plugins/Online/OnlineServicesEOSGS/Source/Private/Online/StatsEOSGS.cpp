// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/StatsEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_stats.h"

#define UE_ONLINE_STAT_EOS_DOUBLE_PRECISION (0.001) // NOTE!!! Modifying the precision will result in broken value for existing stats of all users!
#define UE_ONLINE_STAT_EOS_NAME_MAX_LENGTH (256 + 1) // 256 plus null terminator

namespace UE::Online {

struct FStatNameRawBufferEOS
{
	char StatName[UE_ONLINE_STAT_EOS_NAME_MAX_LENGTH];
};

FStatsEOSGS::FStatsEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

void FStatsEOSGS::Initialize()
{
	Super::Initialize();

	StatsHandle = EOS_Platform_GetStatsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(StatsHandle);
}

TOnlineAsyncOpHandle<FUpdateStats> FStatsEOSGS::UpdateStats(FUpdateStats::Params&& Params)
{
	TOnlineAsyncOpRef<FUpdateStats> Op = GetOp<FUpdateStats>(MoveTemp(Params));

	if (!Services.Get<IAuth>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	for (const FUserStats& UpdateUserStats : Op->GetParams().UpdateUsersStats)
	{
		if (Op->GetParams().LocalAccountId != UpdateUserStats.AccountId)
		{
			// Client can only update the stats of the local account itself
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}
	}

	for (const FUserStats& UpdateUserStats : Op->GetParams().UpdateUsersStats)
	{
		Op->Then([this, &UpdateUserStats](TOnlineAsyncOp<FUpdateStats>& InAsyncOp, TPromise<const EOS_Stats_IngestStatCompleteCallbackInfo*>&& Promise)
		{
			TArray<EOS_Stats_IngestData> EOSData;
			TArray<FStatNameRawBufferEOS> EOSStatNames;

			// Preallocate all of the memory
			EOSData.AddZeroed(UpdateUserStats.Stats.Num());
			EOSStatNames.AddZeroed(UpdateUserStats.Stats.Num());

			uint32 Index = 0;
			for (const TPair<FString, FStatValue>& StatPair : UpdateUserStats.Stats)
			{
				EOS_Stats_IngestData& EOSStat = EOSData[Index];
				EOSStat.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_STATS_INGESTDATA_API_LATEST, 1);

				const FStatDefinition* StatDefinition = GetStatDefinition(StatPair.Key);
				check(StatDefinition);
				check(StatPair.Value.GetType() == StatDefinition->DefaultValue.GetType());

				switch (StatPair.Value.GetType())
				{
				case ESchemaAttributeType::Bool:
					EOSStat.IngestAmount = StatPair.Value.GetBoolean();
					break;
				case ESchemaAttributeType::Double:
				{
					double ConvertedDouble = StatPair.Value.GetDouble() / UE_ONLINE_STAT_EOS_DOUBLE_PRECISION;
					double MinLimit = static_cast<double>(TNumericLimits<int32>::Min());
					double MaxLimit = static_cast<double>(TNumericLimits<int32>::Max());
					check(ConvertedDouble > MinLimit && ConvertedDouble < MaxLimit);
					EOSStat.IngestAmount = static_cast<int32>(FMath::Clamp(ConvertedDouble, MinLimit, MaxLimit));
					break;
				}
				case ESchemaAttributeType::Int64:
				{
					int64 MinLimit = static_cast<int64>(TNumericLimits<int32>::Min());
					int64 MaxLimit = static_cast<int64>(TNumericLimits<int32>::Max());
					check(StatPair.Value.GetInt64() > MinLimit && StatPair.Value.GetInt64() < MaxLimit);
					EOSStat.IngestAmount = static_cast<int32>(FMath::Clamp(StatPair.Value.GetInt64(), MinLimit, MaxLimit));
					break;
				}
				default: checkNoEntry(); // Intentional fallthrough
				case ESchemaAttributeType::String: // Not supported
					break;
				}

				FCStringAnsi::Strncpy(EOSStatNames[Index].StatName, TCHAR_TO_UTF8(*StatPair.Key), UE_ONLINE_STAT_EOS_NAME_MAX_LENGTH);
				EOSStat.StatName = EOSStatNames[Index].StatName;

				Index++;
			}

			EOS_Stats_IngestStatOptions Options = { };
			Options.ApiVersion = 3;
			UE_EOS_CHECK_API_MISMATCH(EOS_STATS_INGESTSTAT_API_LATEST, 3);
			Options.LocalUserId = GetProductUserIdChecked(InAsyncOp.GetParams().LocalAccountId);
			Options.TargetUserId = GetProductUserIdChecked(UpdateUserStats.AccountId);
			Options.Stats = EOSData.GetData();
			Options.StatsCount = EOSData.Num();

			EOS_Async(EOS_Stats_IngestStat, StatsHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FUpdateStats>& InAsyncOp, const EOS_Stats_IngestStatCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Stats_IngestStat failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}
		});
	}

	Op->Then([this](TOnlineAsyncOp<FUpdateStats>& InAsyncOp)
	{
		InAsyncOp.SetResult({});
		OnStatsUpdatedEvent.Broadcast(InAsyncOp.GetParams());
	});

	Op->Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

namespace Private
{

void QueryStatsEOS(EOnlineServices Services, EOS_HStats StatsHandle, const FAccountId& LocalAccountId, const FAccountId& TargetAccountId, const TArray<FString>& StatNames, TPromise<const EOS_Stats_OnQueryStatsCompleteCallbackInfo*>&& Promise)
{
	EOS_Stats_QueryStatsOptions Options;
	Options.ApiVersion = 3;
	UE_EOS_CHECK_API_MISMATCH(EOS_STATS_INGESTSTAT_API_LATEST, 3);
	Options.StartTime = EOS_STATS_TIME_UNDEFINED;
	Options.EndTime = EOS_STATS_TIME_UNDEFINED;

	TArray<FStatNameRawBufferEOS> EOSStatNames;
	TArray<char*> EOSStatNamesPtr;

	EOSStatNames.AddZeroed(StatNames.Num());
	EOSStatNamesPtr.AddZeroed(StatNames.Num());

	uint32 Index = 0;
	for (const FString& StatName : StatNames)
	{
		FCStringAnsi::Strncpy(EOSStatNames[Index].StatName, TCHAR_TO_UTF8(*StatName), UE_ONLINE_STAT_EOS_NAME_MAX_LENGTH);
		EOSStatNamesPtr[Index] = EOSStatNames[Index].StatName;
		Index++;
	}

	Options.StatNames = (const char**)EOSStatNamesPtr.GetData();
	Options.StatNamesCount = StatNames.Num();
	Options.LocalUserId = GetProductUserIdChecked(LocalAccountId);
	Options.TargetUserId = GetProductUserIdChecked(TargetAccountId);

	EOS_Async(EOS_Stats_QueryStats, StatsHandle, Options, MoveTemp(Promise));
}

}

void FStatsEOSGS::ReadStatsFromEOSResult(const EOS_Stats_OnQueryStatsCompleteCallbackInfo* Data, const TArray<FString>& StatNames, TMap<FString, FStatValue>& OutStats)
{
	for (const FString& StatName : StatNames)
	{
		char StatNameANSI[UE_ONLINE_STAT_EOS_NAME_MAX_LENGTH];
		EOS_Stats_CopyStatByNameOptions Options = { };
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_STATS_COPYSTATBYNAME_API_LATEST, 1);
		Options.TargetUserId = Data->TargetUserId;
		Options.Name = StatNameANSI;
		FCStringAnsi::Strncpy(StatNameANSI, TCHAR_TO_UTF8(*StatName), UE_ONLINE_STAT_EOS_NAME_MAX_LENGTH);

		EOS_Stats_Stat* ReadStat = nullptr;

		const FStatDefinition* StatDefinition = GetStatDefinition(StatName);
		check(StatDefinition);

		FStatValue StatValue = StatDefinition->DefaultValue;
		if (EOS_Stats_CopyStatByName(StatsHandle, &Options, &ReadStat) == EOS_EResult::EOS_Success)
		{
			switch (StatDefinition->DefaultValue.GetType())
			{
				case ESchemaAttributeType::Bool:
					StatValue.Set(static_cast<bool>(ReadStat->Value != 0));
					break;
				case ESchemaAttributeType::Double:
					StatValue.Set(static_cast<double>(ReadStat->Value) * UE_ONLINE_STAT_EOS_DOUBLE_PRECISION);
					break;
				case ESchemaAttributeType::Int64:
					StatValue.Set(static_cast<int64>(ReadStat->Value));
					break;
				default: checkNoEntry(); // Intentional fallthrough
				case ESchemaAttributeType::String: // Not supported
					break;
			}
			OutStats.Add(StatName, FStatValue(static_cast<int64>(ReadStat->Value)));
			EOS_Stats_Stat_Release(ReadStat);
		}
		else
		{
			// Put an empty stat in
			UE_LOGF(LogOnlineServices, VeryVerbose, "Value not found for stat %ls, adding empty value", *StatName);
		}
		OutStats.Add(StatName, MoveTemp(StatValue));
	}
}

TOnlineAsyncOpHandle<FQueryStats> FStatsEOSGS::QueryStats(FQueryStats::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryStats> Op = GetJoinableOp<FQueryStats>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FQueryStats>& InAsyncOp)
		{
			const FQueryStats::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FQueryStats>& InAsyncOp, TPromise<const EOS_Stats_OnQueryStatsCompleteCallbackInfo*>&& Promise)
		{
			const FQueryStats::Params& Params = InAsyncOp.GetParams();
			Private::QueryStatsEOS(Services.GetServicesProvider(), StatsHandle, Params.LocalAccountId, Params.TargetAccountId, Params.StatNames, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FQueryStats>& InAsyncOp, const EOS_Stats_OnQueryStatsCompleteCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_Stats_QueryStats failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			const FQueryStats::Params& Params = InAsyncOp.GetParams();

			FQueryStats::Result Result;
			ReadStatsFromEOSResult(Data, Params.StatNames, Result.Stats);

			FUserStats UserStats;
			UserStats.AccountId = Params.TargetAccountId;
			UserStats.Stats = Result.Stats;
			CacheUserStats(UserStats);

			InAsyncOp.SetResult(MoveTemp(Result));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FBatchQueryStats> FStatsEOSGS::BatchQueryStats(FBatchQueryStats::Params&& Params)
{
	TOnlineAsyncOpRef<FBatchQueryStats> Op = GetJoinableOp<FBatchQueryStats>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FBatchQueryStats>& InAsyncOp)
		{
			const FBatchQueryStats::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
			else if (Params.TargetAccountIds.IsEmpty() || Params.StatNames.IsEmpty())
			{
				InAsyncOp.SetError(Errors::InvalidParams());
			}
		});

		for (const FAccountId& TargetAccountId : Op->GetParams().TargetAccountIds)
		{
			Op->Then([this, TargetAccountId](TOnlineAsyncOp<FBatchQueryStats>& InAsyncOp, TPromise<const EOS_Stats_OnQueryStatsCompleteCallbackInfo*>&& Promise)
			{
				Private::QueryStatsEOS(Services.GetServicesProvider(), StatsHandle, InAsyncOp.GetParams().LocalAccountId, TargetAccountId, InAsyncOp.GetParams().StatNames, MoveTemp(Promise));
			})
			.Then([this, TargetAccountId](TOnlineAsyncOp<FBatchQueryStats>& InAsyncOp, const EOS_Stats_OnQueryStatsCompleteCallbackInfo* Data)
			{
				if (Data->ResultCode != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_Stats_QueryStats failed with result=[%ls]", *LexToString(Data->ResultCode));
					BatchQueriedUsersStats.Reset();
					InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
					return;
				}

				FUserStats& UserStats = BatchQueriedUsersStats.Emplace_GetRef();
				UserStats.AccountId = TargetAccountId;

				ReadStatsFromEOSResult(Data, InAsyncOp.GetParams().StatNames, UserStats.Stats);
				CacheUserStats(UserStats);
			});
		}

		Op->Then([this](TOnlineAsyncOp<FBatchQueryStats>& InAsyncOp)
		{
			FBatchQueryStats::Result Result;
			Result.UsersStats = MoveTemp(BatchQueriedUsersStats);
			InAsyncOp.SetResult(MoveTemp(Result));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

/* UE::Online */ }
