// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/TitleFileEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_titlestorage.h"


namespace UE::Online {

FTitleFileEOSGS::FTitleFileEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

void FTitleFileEOSGS::Initialize()
{
	Super::Initialize();

	TitleStorageHandle = EOS_Platform_GetTitleStorageInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(TitleStorageHandle);
}

void FTitleFileEOSGS::UpdateConfig()
{
	Super::UpdateConfig();
	LoadConfig(Config);
}

TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> FTitleFileEOSGS::EnumerateFiles(FTitleFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FTitleFileEnumerateFiles> Op = GetJoinableOp<FTitleFileEnumerateFiles>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& InAsyncOp)
		{
			const FTitleFileEnumerateFiles::Params& Params = InAsyncOp.GetParams();

			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
			else if (Config.SearchTag.IsEmpty())
			{
				InAsyncOp.SetError(Errors::NotConfigured());
			}
		})
		.Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& InAsyncOp, TPromise<const EOS_TitleStorage_QueryFileListCallbackInfo*>&& Promise)
		{
			const FTitleFileEnumerateFiles::Params& Params = InAsyncOp.GetParams();

			const FTCHARToUTF8 Utf8SearchTag(*Config.SearchTag);
			const char* SearchTagPtr = Utf8SearchTag.Get();

			EOS_TitleStorage_QueryFileListOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_QUERYFILELISTOPTIONS_API_LATEST, 1);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.ListOfTagsCount = 1;
			Options.ListOfTags = &SearchTagPtr;

			EOS_Async(EOS_TitleStorage_QueryFileList, TitleStorageHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& InAsyncOp, const EOS_TitleStorage_QueryFileListCallbackInfo* Data)
		{
			if (Data->ResultCode != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_TitleStorage_QueryFileList failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			const FTitleFileEnumerateFiles::Params& Params = InAsyncOp.GetParams();
			const EOS_ProductUserId LocalUserPuid = GetProductUserIdChecked(Params.LocalAccountId);

			EOS_TitleStorage_GetFileMetadataCountOptions GetCountOptions;
			GetCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_GETFILEMETADATACOUNTOPTIONS_API_LATEST, 1);
			GetCountOptions.LocalUserId = LocalUserPuid;

			const uint32_t NumFiles = EOS_TitleStorage_GetFileMetadataCount(TitleStorageHandle, &GetCountOptions);

			TArray<FString> NewEnumeratedFiles;
			NewEnumeratedFiles.Reserve(NumFiles);

			for (uint32_t FileIdx = 0; FileIdx < NumFiles; FileIdx++)
			{
				EOS_TitleStorage_CopyFileMetadataAtIndexOptions CopyOptions;
				CopyOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_COPYFILEMETADATAATINDEXOPTIONS_API_LATEST, 1);
				CopyOptions.LocalUserId = LocalUserPuid;
				CopyOptions.Index = FileIdx;

				EOS_TitleStorage_FileMetadata* FileMetadata = nullptr;

				const EOS_EResult Result = EOS_TitleStorage_CopyFileMetadataAtIndex(TitleStorageHandle, &CopyOptions, &FileMetadata);
				if (Result != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_TitleStorage_CopyFileMetadataAtIndex failed with result=[%ls]", *LexToString(Result));
					InAsyncOp.SetError(Errors::FromEOSResult(Result));
					return;
				}

				NewEnumeratedFiles.Emplace(UTF8_TO_TCHAR(FileMetadata->Filename));

				EOS_TitleStorage_FileMetadata_Release(FileMetadata);
			}

			EnumeratedFiles.Emplace(MoveTemp(NewEnumeratedFiles));
			InAsyncOp.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FTitleFileGetEnumeratedFiles> FTitleFileEOSGS::GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	if (!EnumeratedFiles.IsSet())
	{
		// Call EnumerateFiles first
		return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	return TOnlineResult<FTitleFileGetEnumeratedFiles>({*EnumeratedFiles});
}

struct FTitleFileReadFileClientData
{
	using PromiseType = TPromise<const EOS_TitleStorage_ReadFileCallbackInfo*>;

	FTitleFileReadFileClientData(const FTitleFileContentsRef& InFileContents, PromiseType&& InPromise)
		: FileContents(InFileContents)
		, Promise(MoveTemp(InPromise))
	{}

	FTitleFileContentsWeakPtr FileContents;
	PromiseType Promise;
};

TOnlineAsyncOpHandle<FTitleFileReadFile> FTitleFileEOSGS::ReadFile(FTitleFileReadFile::Params&& InParams)
{
	static const TCHAR* FileContentsKey = TEXT("FileContents");
	static const TCHAR* RequestHandleKey = TEXT("RequestHandle");

	TOnlineAsyncOpRef<FTitleFileReadFile> Op = GetJoinableOp<FTitleFileReadFile>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FTitleFileReadFile::Params& Params = Op->GetParams();
		if (Params.Filename.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FTitleFileReadFile>& InAsyncOp)
		{
			const FTitleFileReadFile::Params& Params = InAsyncOp.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				InAsyncOp.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FTitleFileReadFile>& InAsyncOp, TPromise<const EOS_TitleStorage_ReadFileCallbackInfo*>&& Promise)
		{
			const FTitleFileReadFile::Params& Params = InAsyncOp.GetParams();

			FTitleFileContentsRef FileContentsRef = MakeShared<FTitleFileContents>();
			InAsyncOp.Data.Set<FTitleFileContentsRef>(FileContentsKey, FileContentsRef);

			const FTCHARToUTF8 Utf8Filename(*Params.Filename);

			EOS_TitleStorage_ReadFileOptions Options = {};
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_TITLESTORAGE_READFILEOPTIONS_API_LATEST, 2);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.Filename = Utf8Filename.Get();
			Options.ReadChunkLengthBytes = Config.ReadChunkLengthBytes;
			Options.ReadFileDataCallback = &FTitleFileEOSGS::OnReadFileDataStatic;
			Options.FileTransferProgressCallback = &FTitleFileEOSGS::OnFileTransferProgressStatic;

			FTitleFileReadFileClientData* ClientData = new FTitleFileReadFileClientData(FileContentsRef, MoveTemp(Promise));

			const EOS_HTitleStorageFileTransferRequest RequestHandle = EOS_TitleStorage_ReadFile(TitleStorageHandle, &Options, ClientData, &FTitleFileEOSGS::OnReadFileCompleteStatic);
			InAsyncOp.Data.Set<EOS_HTitleStorageFileTransferRequest>(RequestHandleKey, RequestHandle);
		})
		.Then([this](TOnlineAsyncOp<FTitleFileReadFile>& InAsyncOp, const EOS_TitleStorage_ReadFileCallbackInfo* Data)
		{
			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				const FTitleFileContentsRef* FileContents = InAsyncOp.Data.Get<FTitleFileContentsRef>(FileContentsKey);
				if (ensure(FileContents))
				{
					InAsyncOp.SetResult({ *FileContents });
				}
				else
				{
					InAsyncOp.SetError(Errors::Unknown());
				}
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_TitleStorage_ReadFile failed with result=[%ls]", *LexToString(Data->ResultCode));
				InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
			}

			if (const EOS_HTitleStorageFileTransferRequest* RequestHandle = InAsyncOp.Data.Get<EOS_HTitleStorageFileTransferRequest>(RequestHandleKey))
			{
				EOS_TitleStorageFileTransferRequest_Release(*RequestHandle);
			}
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

EOS_TitleStorage_EReadResult EOS_CALL FTitleFileEOSGS::OnReadFileDataStatic(const EOS_TitleStorage_ReadFileDataCallbackInfo* Data)
{
	FTitleFileReadFileClientData* ClientData = (FTitleFileReadFileClientData*)Data->ClientData;
	check(ClientData);
	FTitleFileContentsPtr FileContentsPtr = ClientData->FileContents.Pin();
	if (ensure(FileContentsPtr))
	{
		FTitleFileContents& FileContents = *ConstCastSharedPtr<FTitleFileContents>(FileContentsPtr);
		// Only has any effect on the first callback
		FileContents.Reserve(Data->TotalFileSizeBytes);
		FileContents.Append((uint8*)Data->DataChunk, Data->DataChunkLengthBytes);

		UE_LOGF(LogOnlineServices, VeryVerbose, "FTitleFileEOSGS::ReadFile ReadProgress Filename=[%ls] %d/%d", UTF8_TO_TCHAR(Data->Filename), FileContents.Num(), Data->TotalFileSizeBytes);
	}

	return EOS_TitleStorage_EReadResult::EOS_TS_RR_ContinueReading;
}
void EOS_CALL FTitleFileEOSGS::OnFileTransferProgressStatic(const EOS_TitleStorage_FileTransferProgressCallbackInfo* Data)
{
	UE_LOGF(LogOnlineServices, VeryVerbose, "FTitleFileEOSGS::ReadFile TransferProgress Filename=[%ls] %d/%d", UTF8_TO_TCHAR(Data->Filename), Data->BytesTransferred, Data->TotalFileSizeBytes);
}

void EOS_CALL FTitleFileEOSGS::OnReadFileCompleteStatic(const EOS_TitleStorage_ReadFileCallbackInfo* Data)
{
	UE_LOGF(LogOnlineServices, VeryVerbose, "FTitleFileEOSGS::ReadFile Complete Filename=[%ls]", UTF8_TO_TCHAR(Data->Filename));

	FTitleFileReadFileClientData* ClientData = (FTitleFileReadFileClientData*)Data->ClientData;
	check(ClientData);
	ClientData->Promise.EmplaceValue(Data);

	delete ClientData;
}

/* UE::Online */ }
