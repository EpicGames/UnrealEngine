// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserFileEOSGS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthEOSGS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/OnlineServicesEOSGS.h"

#include "eos_playerdatastorage.h"


namespace UE::Online {

static const TCHAR* RequestHandleKey = TEXT("RequestHandle");

FUserFileEOSGS::FUserFileEOSGS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

void FUserFileEOSGS::Initialize()
{
	Super::Initialize();

	PlayerDataStorageHandle = EOS_Platform_GetPlayerDataStorageInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(PlayerDataStorageHandle);
}

void FUserFileEOSGS::UpdateConfig()
{
	Super::UpdateConfig();
	LoadConfig(Config);
}

TOnlineAsyncOpHandle<FUserFileEnumerateFiles> FUserFileEOSGS::EnumerateFiles(FUserFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileEnumerateFiles> Op = GetJoinableOp<FUserFileEnumerateFiles>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op)
		{
			const FUserFileEnumerateFiles::Params& Params = Op.GetParams();

			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op, TPromise<const EOS_PlayerDataStorage_QueryFileListCallbackInfo*>&& Promise)
		{
			const FUserFileEnumerateFiles::Params& Params = Op.GetParams();

			EOS_PlayerDataStorage_QueryFileListOptions Options = {};
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_QUERYFILELISTOPTIONS_API_LATEST, 2);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);

			EOS_Async(EOS_PlayerDataStorage_QueryFileList, PlayerDataStorageHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op, const EOS_PlayerDataStorage_QueryFileListCallbackInfo* Data)
		{
			const FUserFileEnumerateFiles::Params& Params = Op.GetParams();

			if (Data->ResultCode != EOS_EResult::EOS_Success
				&& Data->ResultCode != EOS_EResult::EOS_NotFound)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_QueryFileList failed with result=[%ls]", *LexToString(Data->ResultCode));
				Op.SetError(Errors::FromEOSResult(Data->ResultCode));
				return;
			}

			const EOS_ProductUserId LocalUserPuid = GetProductUserIdChecked(Params.LocalAccountId);

			EOS_PlayerDataStorage_GetFileMetadataCountOptions GetCountOptions;
			GetCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_GETFILEMETADATACOUNTOPTIONS_API_LATEST, 1);
			GetCountOptions.LocalUserId = LocalUserPuid;

			int32_t NumFiles;
			EOS_EResult Result = EOS_PlayerDataStorage_GetFileMetadataCount(PlayerDataStorageHandle, &GetCountOptions, &NumFiles);
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_GetFileMetadataCount failed with result=[%ls]", *LexToString(Result));
				Op.SetError(Errors::FromEOSResult(Result));
				return;
			}

			check(NumFiles >= 0);

			TArray<FString> NewEnumeratedFiles;
			NewEnumeratedFiles.Reserve(NumFiles);

			for (int32_t FileIdx = 0; FileIdx < NumFiles; FileIdx++)
			{
				EOS_PlayerDataStorage_CopyFileMetadataAtIndexOptions CopyOptions;
				CopyOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_COPYFILEMETADATAATINDEXOPTIONS_API_LATEST, 1);
				CopyOptions.LocalUserId = LocalUserPuid;
				CopyOptions.Index = FileIdx;

				EOS_PlayerDataStorage_FileMetadata* FileMetadata = nullptr;
				Result = EOS_PlayerDataStorage_CopyFileMetadataAtIndex(PlayerDataStorageHandle, &CopyOptions, &FileMetadata);
				if (Result != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_CopyFileMetadataAtIndex failed with result=[%ls]", *LexToString(Result));
					Op.SetError(Errors::FromEOSResult(Result));
					return;
				}

				NewEnumeratedFiles.Emplace(UTF8_TO_TCHAR(FileMetadata->Filename));

				EOS_PlayerDataStorage_FileMetadata_Release(FileMetadata);
			}

			UserToFiles.Emplace(Params.LocalAccountId, MoveTemp(NewEnumeratedFiles));
			Op.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FUserFileGetEnumeratedFiles> FUserFileEOSGS::GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	const TArray<FString>* Found = UserToFiles.Find(Params.LocalAccountId);
	if (!Found)
	{
		// Call EnumerateFiles first
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	return TOnlineResult<FUserFileGetEnumeratedFiles>({*Found});
}

struct FUserFileReadFileClientData
{
	using PromiseType = TPromise<const EOS_PlayerDataStorage_ReadFileCallbackInfo*>;

	FUserFileReadFileClientData(const FUserFileContentsRef& InFileContents, PromiseType&& InPromise)
		: FileContents(InFileContents)
		, Promise(MoveTemp(InPromise))
	{}

	FUserFileContentsWeakPtr FileContents;
	PromiseType Promise;
};

TOnlineAsyncOpHandle<FUserFileReadFile> FUserFileEOSGS::ReadFile(FUserFileReadFile::Params&& InParams)
{
	static const TCHAR* FileContentsKey = TEXT("FileContents");

	TOnlineAsyncOpRef<FUserFileReadFile> Op = GetJoinableOp<FUserFileReadFile>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FUserFileReadFile::Params& Params = Op->GetParams();
		if (Params.Filename.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op)
		{
			const FUserFileReadFile::Params& Params = Op.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op, TPromise<const EOS_PlayerDataStorage_ReadFileCallbackInfo*>&& Promise)
		{
			const FUserFileReadFile::Params& Params = Op.GetParams();

			FUserFileContentsRef FileContentsRef = MakeShared<FUserFileContents>();
			Op.Data.Set<FUserFileContentsRef>(FileContentsKey, FileContentsRef);

			const auto Utf8Filename = StringCast<UTF8CHAR>(*Params.Filename);

			EOS_PlayerDataStorage_ReadFileOptions Options = {};
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_READFILEOPTIONS_API_LATEST, 2);
			Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
			Options.Filename = (const char*)Utf8Filename.Get();
			Options.ReadChunkLengthBytes = Config.ChunkLengthBytes;
			Options.ReadFileDataCallback = &FUserFileEOSGS::OnReadFileDataStatic;
			Options.FileTransferProgressCallback = &FUserFileEOSGS::OnFileTransferProgressStatic;

			FUserFileReadFileClientData* ClientData = new FUserFileReadFileClientData(FileContentsRef, MoveTemp(Promise));

			const EOS_HPlayerDataStorageFileTransferRequest RequestHandle = EOS_PlayerDataStorage_ReadFile(PlayerDataStorageHandle, &Options, ClientData, &FUserFileEOSGS::OnReadFileCompleteStatic);
			Op.Data.Set<EOS_HPlayerDataStorageFileTransferRequest>(RequestHandleKey, RequestHandle);
		})
		.Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op, const EOS_PlayerDataStorage_ReadFileCallbackInfo* Data)
		{
			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				const FUserFileContentsRef* FileContents = Op.Data.Get<FUserFileContentsRef>(FileContentsKey);
				if (ensure(FileContents))
				{
					Op.SetResult({ *FileContents });
				}
				else
				{
					Op.SetError(Errors::Unknown());
				}
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_ReadFile failed with result=[%ls]", *LexToString(Data->ResultCode));
				Op.SetError(Errors::FromEOSResult(Data->ResultCode));
			}

			if (const EOS_HPlayerDataStorageFileTransferRequest* RequestHandle = Op.Data.Get<EOS_HPlayerDataStorageFileTransferRequest>(RequestHandleKey))
			{
				EOS_PlayerDataStorageFileTransferRequest_Release(*RequestHandle);
			}
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

struct FUserFileWriteFileClientData
{
	using PromiseType = TPromise<const EOS_PlayerDataStorage_WriteFileCallbackInfo*>;

	FUserFileWriteFileClientData(FUserFileContents&& InFileContents, PromiseType&& InPromise)
		: FileContents(MoveTemp(InFileContents))
		, Promise(MoveTemp(InPromise))
	{}

	uint32_t BytesWritten = 0;
	FUserFileContents FileContents;
	PromiseType Promise;
};

TOnlineAsyncOpHandle<FUserFileWriteFile> FUserFileEOSGS::WriteFile(FUserFileWriteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileWriteFile> Op = GetOp<FUserFileWriteFile>(MoveTemp(InParams));
	const FUserFileWriteFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileWriteFile>& Op, TPromise<const EOS_PlayerDataStorage_WriteFileCallbackInfo*>&& Promise)
	{
		const FUserFileWriteFile::Params& Params = Op.GetParams();

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::InvalidUser());
			Promise.EmplaceValue();
			return;
		}

		const auto Utf8Filename = StringCast<UTF8CHAR>(*Params.Filename);

		EOS_PlayerDataStorage_WriteFileOptions Options = {};
		Options.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_WRITEFILEOPTIONS_API_LATEST, 2);
		Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
		Options.Filename = (const char*)Utf8Filename.Get();
		Options.ChunkLengthBytes = Config.ChunkLengthBytes;
		Options.WriteFileDataCallback = &FUserFileEOSGS::OnWriteFileDataStatic;
		Options.FileTransferProgressCallback = &FUserFileEOSGS::OnFileTransferProgressStatic;

		FUserFileWriteFileClientData* ClientData = new FUserFileWriteFileClientData(MoveTemp(const_cast<FUserFileWriteFile::Params&>(Params).FileContents), MoveTemp(Promise));

		const EOS_HPlayerDataStorageFileTransferRequest RequestHandle = EOS_PlayerDataStorage_WriteFile(PlayerDataStorageHandle, &Options, ClientData, &FUserFileEOSGS::OnWriteFileCompleteStatic);
		Op.Data.Set<EOS_HPlayerDataStorageFileTransferRequest>(RequestHandleKey, RequestHandle);
	})
	.Then([this](TOnlineAsyncOp<FUserFileWriteFile>& Op, const EOS_PlayerDataStorage_WriteFileCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			Op.SetResult({});
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_WriteFile failed with result=[%ls]", *LexToString(Data->ResultCode));
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
		}

		if (const EOS_HPlayerDataStorageFileTransferRequest* RequestHandle = Op.Data.Get<EOS_HPlayerDataStorageFileTransferRequest>(RequestHandleKey))
		{
			EOS_PlayerDataStorageFileTransferRequest_Release(*RequestHandle);
		}
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileCopyFile> FUserFileEOSGS::CopyFile(FUserFileCopyFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileCopyFile> Op = GetOp<FUserFileCopyFile>(MoveTemp(InParams));
	const FUserFileCopyFile::Params& Params = Op->GetParams();

	if (Params.SourceFilename.IsEmpty() || Params.TargetFilename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileCopyFile>& Op, TPromise<const EOS_PlayerDataStorage_DuplicateFileCallbackInfo*>&& Promise)
	{
		const FUserFileCopyFile::Params& Params = Op.GetParams();

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::InvalidUser());
			Promise.EmplaceValue();
			return;
		}

		const auto Utf8SourceFilename = StringCast<UTF8CHAR>(*Params.SourceFilename);
		const auto Utf8TargetFilename = StringCast<UTF8CHAR>(*Params.TargetFilename);

		EOS_PlayerDataStorage_DuplicateFileOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_DUPLICATEFILEOPTIONS_API_LATEST, 1);
		Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
		Options.SourceFilename = (const char*)Utf8SourceFilename.Get();
		Options.DestinationFilename = (const char*)Utf8TargetFilename.Get();

		EOS_Async(EOS_PlayerDataStorage_DuplicateFile, PlayerDataStorageHandle, Options, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FUserFileCopyFile>& Op, const EOS_PlayerDataStorage_DuplicateFileCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			Op.SetResult({});
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_DuplicateFile failed with result=[%ls]", *LexToString(Data->ResultCode));
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
		}
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileDeleteFile> FUserFileEOSGS::DeleteFile(FUserFileDeleteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileDeleteFile> Op = GetOp<FUserFileDeleteFile>(MoveTemp(InParams));
	const FUserFileDeleteFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileDeleteFile>& Op, TPromise<const EOS_PlayerDataStorage_DeleteFileCallbackInfo*>&& Promise)
	{
		const FUserFileDeleteFile::Params& Params = Op.GetParams();

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::InvalidUser());
			Promise.EmplaceValue();
			return;
		}

		const auto Utf8Filename = StringCast<UTF8CHAR>(*Params.Filename);

		EOS_PlayerDataStorage_DeleteFileOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_PLAYERDATASTORAGE_DELETEFILEOPTIONS_API_LATEST, 1);
		Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
		Options.Filename = (const char*)Utf8Filename.Get();

		EOS_Async(EOS_PlayerDataStorage_DeleteFile, PlayerDataStorageHandle, Options, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FUserFileDeleteFile>& Op, const EOS_PlayerDataStorage_DeleteFileCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			Op.SetResult({});
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "EOS_PlayerDataStorage_DeleteFile failed with result=[%ls]", *LexToString(Data->ResultCode));
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
		}
	})
	.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

EOS_PlayerDataStorage_EReadResult EOS_CALL FUserFileEOSGS::OnReadFileDataStatic(const EOS_PlayerDataStorage_ReadFileDataCallbackInfo* Data)
{
	FUserFileReadFileClientData* ClientData = (FUserFileReadFileClientData*)Data->ClientData;
	check(ClientData);
	FUserFileContentsPtr FileContentsPtr = ClientData->FileContents.Pin();
	if (ensure(FileContentsPtr))
	{
		FUserFileContents& FileContents = *ConstCastSharedPtr<FUserFileContents>(FileContentsPtr);
		// Only has any effect on the first callback
		FileContents.Reserve(Data->TotalFileSizeBytes);
		FileContents.Append((uint8*)Data->DataChunk, Data->DataChunkLengthBytes);

		UE_LOGF(LogOnlineServices, VeryVerbose, "EOS_PlayerDataStorage_ReadFile Progress Filename=[%ls] %d/%d", UTF8_TO_TCHAR(Data->Filename), FileContents.Num(), Data->TotalFileSizeBytes);
	}

	return EOS_PlayerDataStorage_EReadResult::EOS_RR_ContinueReading;
}

void EOS_CALL FUserFileEOSGS::OnReadFileCompleteStatic(const EOS_PlayerDataStorage_ReadFileCallbackInfo* Data)
{
	UE_LOGF(LogOnlineServices, VeryVerbose, "EOS_PlayerDataStorage_ReadFile Complete Filename=[%ls]", UTF8_TO_TCHAR(Data->Filename));

	FUserFileReadFileClientData* ClientData = (FUserFileReadFileClientData*)Data->ClientData;
	check(ClientData);
	ClientData->Promise.EmplaceValue(Data);

	delete ClientData;
}

EOS_PlayerDataStorage_EWriteResult EOS_CALL FUserFileEOSGS::OnWriteFileDataStatic(const EOS_PlayerDataStorage_WriteFileDataCallbackInfo* Data, void* OutDataBuffer, uint32_t* OutDataWritten)
{
	FUserFileWriteFileClientData* ClientData = (FUserFileWriteFileClientData*)Data->ClientData;
	check(ClientData);
	
	const FUserFileContents& FileContents = ClientData->FileContents;

	const uint32_t BytesWrittenBefore = ClientData->BytesWritten;
	const uint32_t BytesRemaining = FileContents.Num() - BytesWrittenBefore;
	const uint32_t BytesToWrite = FMath::Min(Data->DataBufferLengthBytes, BytesRemaining);

	FMemory::Memcpy(OutDataBuffer, FileContents.GetData() + BytesWrittenBefore, BytesToWrite);
	*OutDataWritten = BytesToWrite;

	const uint32_t BytesWrittenTotal = BytesWrittenBefore + BytesToWrite;
	ClientData->BytesWritten = BytesWrittenTotal;

	UE_LOGF(LogOnlineServices, VeryVerbose, "EOS_PlayerDataStorage_WriteFile Progress Filename=[%ls] %d/%d", UTF8_TO_TCHAR(Data->Filename), BytesWrittenTotal, FileContents.Num());

	const bool bDone = BytesWrittenTotal == FileContents.Num();
	if (bDone)
	{
		return EOS_PlayerDataStorage_EWriteResult::EOS_WR_CompleteRequest;
	}
	else
	{
		return EOS_PlayerDataStorage_EWriteResult::EOS_WR_ContinueWriting;
	}
}

void EOS_CALL FUserFileEOSGS::OnWriteFileCompleteStatic(const EOS_PlayerDataStorage_WriteFileCallbackInfo* Data)
{
	UE_LOGF(LogOnlineServices, VeryVerbose, "EOS_PlayerDataStorage_WriteFile Complete Filename=[%ls]", UTF8_TO_TCHAR(Data->Filename));

	FUserFileWriteFileClientData* ClientData = (FUserFileWriteFileClientData*)Data->ClientData;
	check(ClientData);
	ClientData->Promise.EmplaceValue(Data);

	delete ClientData;
}

void EOS_CALL FUserFileEOSGS::OnFileTransferProgressStatic(const EOS_PlayerDataStorage_FileTransferProgressCallbackInfo* Data)
{
	UE_LOGF(LogOnlineServices, VeryVerbose, "FUserFileEOSGS TransferProgress Filename=[%ls] %d/%d", UTF8_TO_TCHAR(Data->Filename), Data->BytesTransferred, Data->TotalFileSizeBytes);
}

/* UE::Online */ }
