// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/AchievementsCommon.h"
#include "Online/OnlineServicesEOSGSTypes.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_achievements_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;

class FAchievementsEOSGS : public FAchievementsCommon
{
public:
	using Super = FAchievementsCommon;

	ONLINESERVICESEOSGS_API FAchievementsEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);
	virtual ~FAchievementsEOSGS() = default;

	// IOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	ONLINESERVICESEOSGS_API virtual void Shutdown() override;

	// IAchievements
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FQueryAchievementDefinitions> QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FGetAchievementIds> GetAchievementIds(FGetAchievementIds::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FGetAchievementDefinition> GetAchievementDefinition(FGetAchievementDefinition::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FQueryAchievementStates> QueryAchievementStates(FQueryAchievementStates::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FGetAchievementState> GetAchievementState(FGetAchievementState::Params&& Params) const override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUnlockAchievements> UnlockAchievements(FUnlockAchievements::Params&& Params) override;

protected:
	ONLINESERVICESEOSGS_API void HandleAchievementsUnlocked(const EOS_Achievements_OnAchievementsUnlockedCallbackV2Info* Data);

	ONLINESERVICESEOSGS_API FAccountId FindAccountId(const EOS_ProductUserId ProductUserId);

	EOS_HAchievements AchievementsHandle = nullptr;

	using FAchievementDefinitionMap = TMap<FString, FAchievementDefinition>;
	TOptional<FAchievementDefinitionMap> AchievementDefinitions;

	FEOSEventRegistrationPtr OnAchievementsUnlocked;
};

/* UE::Online */ }
