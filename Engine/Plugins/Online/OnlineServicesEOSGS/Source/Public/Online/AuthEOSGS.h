// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/AuthCommon.h"
#include "Online/OnlineServicesEOSGSTypes.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_auth_types.h"
#include "eos_connect_types.h"

namespace UE::Online {

class FOnlineServicesEOSGS;
struct FAccountInfoEOS;
class FAccountInfoRegistryEOS;

struct FAuthLoginEASImpl
{
	static constexpr TCHAR Name[] = TEXT("LoginEASImpl");

	struct Params
	{
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		FName CredentialsType;
		FString CredentialsId;
		TVariant<FString, FExternalAuthToken> CredentialsToken;
		TArray<FString> Scopes;
		bool bAutoLinkAccount = true;
	};

	struct Result
	{
		EOS_EpicAccountId EpicAccountId = nullptr;
	};
};

struct FAuthLogoutEASImpl
{
	static constexpr TCHAR Name[] = TEXT("LogoutEASImpl");

	struct Params
	{
		EOS_EpicAccountId EpicAccountId = nullptr;
	};

	struct Result
	{
	};
};

struct FAuthLogoutConnectImpl
{
	static constexpr TCHAR Name[] = TEXT("LogoutConnectImpl");

	struct Params
	{
		EOS_ProductUserId ProductUserId = nullptr;
	};

	struct Result
	{
	};
};

struct FAuthGetExternalAuthTokenImpl
{
	static constexpr TCHAR Name[] = TEXT("GetExternalAuthTokenImpl");

	struct Params
	{
		EOS_EpicAccountId EpicAccountId = nullptr;
	};

	struct Result
	{
		FExternalAuthToken Token;
	};
};

struct FAuthLoginConnectImpl
{
	static constexpr TCHAR Name[] = TEXT("LoginConnectImpl");

	struct Params
	{
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		FExternalAuthToken ExternalAuthToken;
	};

	struct Result
	{
		EOS_ProductUserId ProductUserId = nullptr;
	};
};

struct FAuthConnectLoginRecoveryImpl
{
	static constexpr TCHAR Name[] = TEXT("ConnectLoginRecovery");

	struct Params
	{
		/** The Epic Account ID of the local user whose connect login should be recovered. */
		EOS_EpicAccountId LocalUserId = nullptr;
	};

	struct Result
	{
	};
};

struct FAuthHandleConnectLoginStatusChangedImpl
{
	static constexpr TCHAR Name[] = TEXT("HandleConnectLoginStatusChangedImpl");

	struct Params
	{
		/** The Product User ID of the local player whose status has changed. */
		EOS_ProductUserId LocalUserId = nullptr;
		/** The status prior to the change. */
		EOS_ELoginStatus PreviousStatus = EOS_ELoginStatus::EOS_LS_NotLoggedIn;
		/** The status at the time of the notification. */
		EOS_ELoginStatus CurrentStatus = EOS_ELoginStatus::EOS_LS_NotLoggedIn;
	};

	struct Result
	{
	};
};

struct FAuthHandleConnectAuthNotifyExpirationImpl
{
	static constexpr TCHAR Name[] = TEXT("HandleConnectAuthNotifyExpirationImpl");

	struct Params
	{
		/** The Product User ID of the local player whose status has changed. */
		EOS_ProductUserId LocalUserId = nullptr;
	};

	struct Result
	{
	};
};

struct FAuthHandleEASLoginStatusChangedImpl
{
	static constexpr TCHAR Name[] = TEXT("HandleEASLoginStatusChangedImpl");

	struct Params
	{
		/** The Epic Account ID of the local user whose status has changed */
		EOS_EpicAccountId LocalUserId = nullptr;
		/** The status prior to the change */
		EOS_ELoginStatus PreviousStatus = EOS_ELoginStatus::EOS_LS_NotLoggedIn;
		/** The status at the time of the notification */
		EOS_ELoginStatus CurrentStatus = EOS_ELoginStatus::EOS_LS_NotLoggedIn;
	};

	struct Result
	{
	};
};

struct FAccountInfoEOS final : public FAccountInfo
{
	FTSTicker::FDelegateHandle RestoreLoginTimer;
	EOS_EpicAccountId EpicAccountId = nullptr;
	EOS_ProductUserId ProductUserId = nullptr;
};

class FAccountInfoRegistryEOS final : public FAccountInfoRegistry
{
public:
	using Super = FAccountInfoRegistry;

	virtual ~FAccountInfoRegistryEOS() = default;

	ONLINESERVICESEOSGS_API TSharedPtr<FAccountInfoEOS> Find(FPlatformUserId PlatformUserId) const;
	ONLINESERVICESEOSGS_API TSharedPtr<FAccountInfoEOS> Find(FAccountId AccountId) const;
	ONLINESERVICESEOSGS_API TSharedPtr<FAccountInfoEOS> Find(EOS_EpicAccountId EpicAccountId) const;
	ONLINESERVICESEOSGS_API TSharedPtr<FAccountInfoEOS> Find(EOS_ProductUserId ProductUserId) const;

	ONLINESERVICESEOSGS_API void Register(const TSharedRef<FAccountInfoEOS>& UserAuthData);
	ONLINESERVICESEOSGS_API void Unregister(FAccountId AccountId);

protected:
	ONLINESERVICESEOSGS_API virtual void DoRegister(const TSharedRef<FAccountInfo>& AccountInfo);
	ONLINESERVICESEOSGS_API virtual void DoUnregister(const TSharedRef<FAccountInfo>& AccountInfo);

private:
	TMap<EOS_EpicAccountId, TSharedRef<FAccountInfoEOS>> AuthDataByEpicAccountId;
	TMap<EOS_ProductUserId, TSharedRef<FAccountInfoEOS>> AuthDataByProductUserId;
};

class FAuthEOSGS : public FAuthCommon
{
public:
	using Super = FAuthCommon;

	ONLINESERVICESEOSGS_API FAuthEOSGS(FOnlineServicesCommon& InOwningSubsystem);
	virtual ~FAuthEOSGS() = default;

	// Begin IOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	// End IOnlineComponent

	// Begin IAuth
	ONLINESERVICESEOSGS_API virtual void PreShutdown() override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FAuthLogin> Login(FAuthLogin::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FAuthLogout> Logout(FAuthLogout::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FAuthQueryVerifiedAuthTicket> QueryVerifiedAuthTicket(FAuthQueryVerifiedAuthTicket::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FAuthCancelVerifiedAuthTicket> CancelVerifiedAuthTicket(FAuthCancelVerifiedAuthTicket::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FAuthBeginVerifiedAuthSession> BeginVerifiedAuthSession(FAuthBeginVerifiedAuthSession::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FAuthEndVerifiedAuthSession> EndVerifiedAuthSession(FAuthEndVerifiedAuthSession::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FAuthGetRelyingParty> GetRelyingParty(FAuthGetRelyingParty::Params&& Params) const override;
	// End IAuth

protected:
	// internal operations.

	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FAuthLoginEASImpl>> LoginEASImpl(const FAuthLoginEASImpl::Params& LoginParams);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FAuthLogoutEASImpl>> LogoutEASImpl(const FAuthLogoutEASImpl::Params& LogoutParams);
	ONLINESERVICESEOSGS_API TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> GetExternalAuthTokenImpl(const FAuthGetExternalAuthTokenImpl::Params& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FAuthLoginConnectImpl>> LoginConnectImpl(const FAuthLoginConnectImpl::Params& LoginParams);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FAuthLogoutConnectImpl>> LogoutConnectImpl(const FAuthLogoutConnectImpl::Params& LogoutParams);
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FAuthConnectLoginRecoveryImpl> ConnectLoginRecoveryImplOp(FAuthConnectLoginRecoveryImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FAuthHandleConnectLoginStatusChangedImpl> HandleConnectLoginStatusChangedImplOp(FAuthHandleConnectLoginStatusChangedImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FAuthHandleConnectAuthNotifyExpirationImpl> HandleConnectAuthNotifyExpirationImplOp(FAuthHandleConnectAuthNotifyExpirationImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FAuthHandleEASLoginStatusChangedImpl> HandleEASLoginStatusChangedImplOp(FAuthHandleEASLoginStatusChangedImpl::Params&& Params);

protected:
	// Service event handling.
	ONLINESERVICESEOSGS_API void RegisterHandlers();
	ONLINESERVICESEOSGS_API void UnregisterHandlers();
	ONLINESERVICESEOSGS_API void OnConnectLoginStatusChanged(const EOS_Connect_LoginStatusChangedCallbackInfo* Data);
	ONLINESERVICESEOSGS_API void OnConnectAuthNotifyExpiration(const EOS_Connect_AuthExpirationCallbackInfo* Data);
	ONLINESERVICESEOSGS_API void OnEASLoginStatusChanged(const EOS_Auth_LoginStatusChangedCallbackInfo* Data);

protected:
	template <typename ValueType>
	using TLocalUserArray = TSparseArray<ValueType, TInlineSparseArrayAllocator<MAX_LOCAL_PLAYERS>>;

	struct FLoginContinuationData
	{
		FLoginContinuationId ContinuationId;
		EOS_ContinuanceToken ContinuanceToken;
		EOS_ELinkAccountFlags LinkAccountFlags = EOS_ELinkAccountFlags::EOS_LA_NoFlags;
	};

	struct FUserScopedData
	{
		FLoginContinuationId LastLoginContinuationId;
		TArray<FLoginContinuationData> LoginContinuations;
	};

	uint32 NextLoginContinuationId = 1;

#if !UE_BUILD_SHIPPING
	static ONLINESERVICESEOSGS_API void CheckMetadata();
#endif

	ONLINESERVICESEOSGS_API virtual const FAccountInfoRegistry& GetAccountInfoRegistry() const override;

	ONLINESERVICESEOSGS_API void InitializeConnectLoginRecoveryTimer(const TSharedRef<FAccountInfoEOS>& UserAuthData);

	static ONLINESERVICESEOSGS_API FAccountId CreateAccountId(const EOS_ProductUserId ProductUserId);

	ONLINESERVICESEOSGS_API FUserScopedData* GetUserScopedData(FPlatformUserId PlatformUserId);
	ONLINESERVICESEOSGS_API const FUserScopedData* GetUserScopedData(FPlatformUserId PlatformUserId) const;
	ONLINESERVICESEOSGS_API FUserScopedData* GetOrCreateUserScopedData(FPlatformUserId PlatformUserId);

	EOS_HAuth AuthHandle = nullptr;
	EOS_HConnect ConnectHandle = nullptr;
	FEOSEventRegistrationPtr OnConnectLoginStatusChangedEOSEventRegistration;
	FEOSEventRegistrationPtr OnConnectAuthNotifyExpirationEOSEventRegistration;
	FEOSEventRegistrationPtr OnAuthLoginStatusChangedEOSEventRegistration;
	FAccountInfoRegistryEOS AccountInfoRegistryEOS;
	TLocalUserArray<FUserScopedData> UserScopedData;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAuthLoginEASImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Params, CredentialsType),
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Params, CredentialsId),
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Params, CredentialsToken),
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Params, Scopes),
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Params, bAutoLinkAccount)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLoginEASImpl::Result)
	ONLINE_STRUCT_FIELD(FAuthLoginEASImpl::Result, EpicAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogoutEASImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthLogoutEASImpl::Params, EpicAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogoutEASImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogoutConnectImpl::Params)
ONLINE_STRUCT_FIELD(FAuthLogoutConnectImpl::Params, ProductUserId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogoutConnectImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetExternalAuthTokenImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthGetExternalAuthTokenImpl::Params, EpicAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetExternalAuthTokenImpl::Result)
	ONLINE_STRUCT_FIELD(FAuthGetExternalAuthTokenImpl::Result, Token)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLoginConnectImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthLoginConnectImpl::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAuthLoginConnectImpl::Params, ExternalAuthToken)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLoginConnectImpl::Result)
	ONLINE_STRUCT_FIELD(FAuthLoginConnectImpl::Result, ProductUserId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthConnectLoginRecoveryImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthConnectLoginRecoveryImpl::Params, LocalUserId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthConnectLoginRecoveryImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleConnectLoginStatusChangedImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthHandleConnectLoginStatusChangedImpl::Params, LocalUserId),
	ONLINE_STRUCT_FIELD(FAuthHandleConnectLoginStatusChangedImpl::Params, PreviousStatus),
	ONLINE_STRUCT_FIELD(FAuthHandleConnectLoginStatusChangedImpl::Params, CurrentStatus)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleConnectLoginStatusChangedImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleConnectAuthNotifyExpirationImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthHandleConnectAuthNotifyExpirationImpl::Params, LocalUserId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleConnectAuthNotifyExpirationImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleEASLoginStatusChangedImpl::Params)
	ONLINE_STRUCT_FIELD(FAuthHandleEASLoginStatusChangedImpl::Params, LocalUserId),
	ONLINE_STRUCT_FIELD(FAuthHandleEASLoginStatusChangedImpl::Params, PreviousStatus),
	ONLINE_STRUCT_FIELD(FAuthHandleEASLoginStatusChangedImpl::Params, CurrentStatus)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthHandleEASLoginStatusChangedImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAccountInfoEOS)
	ONLINE_STRUCT_FIELD(FAccountInfoEOS, AccountId),
	ONLINE_STRUCT_FIELD(FAccountInfoEOS, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAccountInfoEOS, LoginStatus),
	ONLINE_STRUCT_FIELD(FAccountInfoEOS, Attributes),
	ONLINE_STRUCT_FIELD(FAccountInfoEOS, EpicAccountId),
	ONLINE_STRUCT_FIELD(FAccountInfoEOS, ProductUserId)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
