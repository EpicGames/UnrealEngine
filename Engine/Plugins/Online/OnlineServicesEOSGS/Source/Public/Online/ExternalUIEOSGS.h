// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/ExternalUICommon.h"
#include "Online/OnlineServicesEOSGSTypes.h"
#include "Online/OnlineServicesEpicCommon.h"

#include "eos_ui_types.h"

#define UE_API ONLINESERVICESEOSGS_API

namespace UE::Online {

class FOnlineServicesEpicCommon;

struct FExternalUIProcessDisplaySettingsUpdatedImp
{
	static constexpr TCHAR Name[] = TEXT("ProcessDisplaySettingsUpdatedImp");

	struct Params
	{
		/** True when any portion of the overlay is visible. */
		bool bIsVisible = false;
		/**
		 * True when the overlay has switched to exclusive input mode.
		 * While in exclusive input mode, no keyboard or mouse input will be sent to the game.
		 */
		bool bIsExclusiveInput = false;
	};

	struct Result
	{
	};
};

class FExternalUIEOSGS : public FExternalUICommon
{
public:
	using Super = FExternalUICommon;

	UE_API FExternalUIEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);
	UE_API virtual ~FExternalUIEOSGS();

	UE_API virtual void Initialize() override;
	UE_API virtual void PreShutdown() override;

protected:
	UE_API void RegisterEventHandlers();
	UE_API void UnregisterEventHandlers();

	UE_API void HandleDisplaySettingsUpdated(const EOS_UI_OnDisplaySettingsUpdatedCallbackInfo* Data);

	UE_API TOnlineAsyncOpHandle<FExternalUIProcessDisplaySettingsUpdatedImp> ProcessDisplaySettingsUpdatedImplOp(FExternalUIProcessDisplaySettingsUpdatedImp::Params&& Params);

	EOS_HUI UIInterfaceHandle = nullptr;
	FEOSEventRegistrationPtr OnDisplaySettingsUpdated;
	bool bLastReportedExclusiveInput = false;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FExternalUIProcessDisplaySettingsUpdatedImp::Params)
	ONLINE_STRUCT_FIELD(FExternalUIProcessDisplaySettingsUpdatedImp::Params, bIsVisible),
	ONLINE_STRUCT_FIELD(FExternalUIProcessDisplaySettingsUpdatedImp::Params, bIsExclusiveInput)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalUIProcessDisplaySettingsUpdatedImp::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }

#undef UE_API
