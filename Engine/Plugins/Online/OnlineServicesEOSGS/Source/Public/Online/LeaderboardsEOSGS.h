// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/LeaderboardsCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_leaderboards_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;

class FLeaderboardsEOSGS : public FLeaderboardsCommon
{
public:
	using Super = FLeaderboardsCommon;

	ONLINESERVICESEOSGS_API FLeaderboardsEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);

	// TOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;

	// ILeaderboards
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FReadEntriesForUsers> ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FReadEntriesAroundRank> ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FReadEntriesAroundUser> ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params) override;

private:
	void ReadEntriesInRange(uint32 StartIndex, uint32 EndIndex, TArray<FLeaderboardEntry>& OutEntries);

	EOS_HLeaderboards LeaderboardsHandle = nullptr;
};

/* UE::Online */ }
