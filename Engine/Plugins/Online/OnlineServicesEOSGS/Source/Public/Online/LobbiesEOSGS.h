// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/LobbiesCommon.h"
#include "OnlineServicesEOSGSTypes.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_lobby_types.h"

namespace UE::Online {

struct FLobbyPrerequisitesEOS;
class FLobbyDataEOS;
class FLobbyDataRegistryEOS;
class FLobbyInviteDataEOS;
class FOnlineServicesEpicCommon;
class FLobbySearchEOS;
struct FClientLobbyDataChanges;
struct FClientLobbyMemberDataChanges;

struct FLobbiesLeaveLobbyImpl
{
	static constexpr TCHAR Name[] = TEXT("LeaveLobbyImpl");

	struct Params
	{
		// The lobby handle data.
		TSharedPtr<FLobbyDataEOS> LobbyData;

		// The local user agent which will perform the action.
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FLobbiesDestroyLobbyImpl
{
	static constexpr TCHAR Name[] = TEXT("DestroyLobbyImpl");

	struct Params
	{
		// The name of the lobby to be destroyed.
		FString LobbyIdString;

		// The local user agent which will perform the action.
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FLobbiesInviteLobbyMemberImpl
{
	static constexpr TCHAR Name[] = TEXT("InviteLobbyMemberImpl");

	struct Params
	{
		// The lobby handle data.
		TSharedPtr<FLobbyDataEOS> LobbyData;

		// The local user agent which will perform the action.
		FAccountId LocalAccountId;

		// The target user for the invitation.
		FAccountId TargetAccountId;
	};

	struct Result
	{
	};
};

struct FLobbiesDeclineLobbyInvitationImpl
{
	static constexpr TCHAR Name[] = TEXT("DeclineLobbyInvitationImpl");

	struct Params
	{
		// The local user agent which will perform the action.
		FAccountId LocalAccountId;

		// Id of the lobby for which the invitations will be declined.
		FLobbyId LobbyId;
	};

	struct Result
	{
	};
};

struct FLobbiesKickLobbyMemberImpl
{
	static constexpr TCHAR Name[] = TEXT("KickLobbyMemberImpl");

	struct Params
	{
		// The lobby handle data.
		TSharedPtr<FLobbyDataEOS> LobbyData;

		// The local user agent which will perform the action.
		FAccountId LocalAccountId;

		// The target user to be kicked.
		FAccountId TargetAccountId;
	};

	struct Result
	{
	};
};

struct FLobbiesPromoteLobbyMemberImpl
{
	static constexpr TCHAR Name[] = TEXT("PromoteLobbyMemberImpl");

	struct Params
	{
		// The lobby handle data.
		TSharedPtr<FLobbyDataEOS> LobbyData;

		// The local user agent which will perform the action.
		FAccountId LocalAccountId;

		// The target user to be promoted to owner.
		FAccountId TargetAccountId;
	};

	struct Result
	{
	};
};

struct FLobbiesModifyLobbyDataImpl
{
	static constexpr TCHAR Name[] = TEXT("ModifyLobbyDataImpl");

	struct Params
	{
		// The lobby handle data.
		TSharedPtr<FLobbyDataEOS> LobbyData;

		// The local user agent which will perform the action.
		FAccountId LocalAccountId;

		/** Translated changes to be applied to the service. */
		FLobbyClientServiceChanges ServiceChanges;
	};

	struct Result
	{
	};
};

struct FLobbiesProcessLobbyNotificationImpl
{
	static constexpr TCHAR Name[] = TEXT("ProcessLobbyNotificationImpl");

	struct Params
	{
		// The lobby handle data.
		TSharedPtr<FLobbyDataEOS> LobbyData;

		// Joining / mutated members.
		TSet<EOS_ProductUserId> MutatedMembers;

		// Leaving members.
		TMap<EOS_ProductUserId, ELobbyMemberLeaveReason> LeavingMembers;
	};

	// Todo: += operator.
	// Mergeable op must be queued by lobby id.

	struct Result
	{
	};
};

class FLobbiesEOSGS : public FLobbiesCommon
{
public:
	using Super = FLobbiesCommon;

	ONLINESERVICESEOSGS_API FLobbiesEOSGS(FOnlineServicesEpicCommon& InServices);

	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	ONLINESERVICESEOSGS_API virtual void PreShutdown() override;

	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FCreateLobby> CreateLobby(FCreateLobby::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FFindLobbies> FindLobbies(FFindLobbies::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FJoinLobby> JoinLobby(FJoinLobby::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FLeaveLobby> LeaveLobby(FLeaveLobby::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FInviteLobbyMember> InviteLobbyMember(FInviteLobbyMember::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FDeclineLobbyInvitation> DeclineLobbyInvitation(FDeclineLobbyInvitation::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FKickLobbyMember> KickLobbyMember(FKickLobbyMember::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FPromoteLobbyMember> PromoteLobbyMember(FPromoteLobbyMember::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FModifyLobbyJoinPolicy> ModifyLobbyJoinPolicy(FModifyLobbyJoinPolicy::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FModifyLobbyAttributes> ModifyLobbyAttributes(FModifyLobbyAttributes::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FModifyLobbyMemberAttributes> ModifyLobbyMemberAttributes(FModifyLobbyMemberAttributes::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FGetJoinedLobbies> GetJoinedLobbies(FGetJoinedLobbies::Params&& Params) override;

private:
	void HandleLobbyUpdated(const EOS_Lobby_LobbyUpdateReceivedCallbackInfo* Data);
	void HandleLobbyMemberUpdated(const EOS_Lobby_LobbyMemberUpdateReceivedCallbackInfo* Data);
	void HandleLobbyMemberStatusReceived(const EOS_Lobby_LobbyMemberStatusReceivedCallbackInfo* Data);
	void HandleLobbyInviteReceived(const EOS_Lobby_LobbyInviteReceivedCallbackInfo* Data);
	void HandleLobbyInviteAccepted(const EOS_Lobby_LobbyInviteAcceptedCallbackInfo* Data);
	void HandleJoinLobbyAccepted(const EOS_Lobby_JoinLobbyAcceptedCallbackInfo* Data);

	FAccountId FindAccountId(const EOS_ProductUserId ProductUserId);

protected:
#if !UE_BUILD_SHIPPING
	static ONLINESERVICESEOSGS_API void CheckMetadata();
#endif

	ONLINESERVICESEOSGS_API void RegisterHandlers();
	ONLINESERVICESEOSGS_API void UnregisterHandlers();

	ONLINESERVICESEOSGS_API void AddActiveLobby(FAccountId LocalAccountId, bool bPresenceEnabled, const TSharedRef<FLobbyDataEOS>& LobbyData);
	ONLINESERVICESEOSGS_API void RemoveActiveLobby(FAccountId LocalAccountId, const TSharedRef<FLobbyDataEOS>& LobbyData);

	// Todo: store list of invites per lobby.
	ONLINESERVICESEOSGS_API void AddActiveInvite(const TSharedRef<FLobbyInviteDataEOS>& Invite);
	ONLINESERVICESEOSGS_API void RemoveActiveInvite(const TSharedRef<FLobbyInviteDataEOS>& Invite);
	ONLINESERVICESEOSGS_API TSharedPtr<FLobbyInviteDataEOS> GetActiveInvite(FAccountId TargetUser, FLobbyId TargetLobbyId);

	// LobbyData will be fetched from the operation data if not set in Params.
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesLeaveLobbyImpl>> LeaveLobbyImpl(FLobbiesLeaveLobbyImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesDestroyLobbyImpl>> DestroyLobbyImpl(FLobbiesDestroyLobbyImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesInviteLobbyMemberImpl>> InviteLobbyMemberImpl(FLobbiesInviteLobbyMemberImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesDeclineLobbyInvitationImpl>> DeclineLobbyInvitationImpl(FLobbiesDeclineLobbyInvitationImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesKickLobbyMemberImpl>> KickLobbyMemberImpl(FLobbiesKickLobbyMemberImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesPromoteLobbyMemberImpl>> PromoteLobbyMemberImpl(FLobbiesPromoteLobbyMemberImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FLobbiesModifyLobbyDataImpl>> ModifyLobbyDataImpl(FLobbiesModifyLobbyDataImpl::Params&& Params);
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FLobbiesProcessLobbyNotificationImpl> ProcessLobbyNotificationImplOp(FLobbiesProcessLobbyNotificationImpl::Params&& Params);

	FEOSEventRegistrationPtr OnLobbyUpdatedEOSEventRegistration;
	FEOSEventRegistrationPtr OnLobbyMemberUpdatedEOSEventRegistration;
	FEOSEventRegistrationPtr OnLobbyMemberStatusReceivedEOSEventRegistration;
	FEOSEventRegistrationPtr OnLobbyInviteReceivedEOSEventRegistration;
	FEOSEventRegistrationPtr OnLobbyInviteAcceptedEOSEventRegistration;
	FEOSEventRegistrationPtr OnJoinLobbyAcceptedEOSEventRegistration;

	TSharedPtr<FLobbyPrerequisitesEOS> LobbyPrerequisites;
	TSharedPtr<FLobbyDataRegistryEOS> LobbyDataRegistry;

	TMap<FAccountId, TSet<TSharedRef<FLobbyDataEOS>>> ActiveLobbies;
	TMap<FAccountId, TMap<FLobbyId, TSharedRef<FLobbyInviteDataEOS>>> ActiveInvites;
	TMap<FAccountId, TSharedRef<FLobbySearchEOS>> ActiveSearchResults;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FLobbiesLeaveLobbyImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesLeaveLobbyImpl::Params, LobbyData),
	ONLINE_STRUCT_FIELD(FLobbiesLeaveLobbyImpl::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesLeaveLobbyImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesDestroyLobbyImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesDestroyLobbyImpl::Params, LobbyIdString),
	ONLINE_STRUCT_FIELD(FLobbiesDestroyLobbyImpl::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesDestroyLobbyImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesInviteLobbyMemberImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesInviteLobbyMemberImpl::Params, LobbyData),
	ONLINE_STRUCT_FIELD(FLobbiesInviteLobbyMemberImpl::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLobbiesInviteLobbyMemberImpl::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesInviteLobbyMemberImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesDeclineLobbyInvitationImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesDeclineLobbyInvitationImpl::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLobbiesDeclineLobbyInvitationImpl::Params, LobbyId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesDeclineLobbyInvitationImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesKickLobbyMemberImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesKickLobbyMemberImpl::Params, LobbyData),
	ONLINE_STRUCT_FIELD(FLobbiesKickLobbyMemberImpl::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLobbiesKickLobbyMemberImpl::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesKickLobbyMemberImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesPromoteLobbyMemberImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesPromoteLobbyMemberImpl::Params, LobbyData),
	ONLINE_STRUCT_FIELD(FLobbiesPromoteLobbyMemberImpl::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLobbiesPromoteLobbyMemberImpl::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesPromoteLobbyMemberImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesModifyLobbyDataImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesModifyLobbyDataImpl::Params, LobbyData),
	ONLINE_STRUCT_FIELD(FLobbiesModifyLobbyDataImpl::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesModifyLobbyDataImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesProcessLobbyNotificationImpl::Params)
	ONLINE_STRUCT_FIELD(FLobbiesProcessLobbyNotificationImpl::Params, LobbyData),
	ONLINE_STRUCT_FIELD(FLobbiesProcessLobbyNotificationImpl::Params, MutatedMembers),
	ONLINE_STRUCT_FIELD(FLobbiesProcessLobbyNotificationImpl::Params, LeavingMembers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbiesProcessLobbyNotificationImpl::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
