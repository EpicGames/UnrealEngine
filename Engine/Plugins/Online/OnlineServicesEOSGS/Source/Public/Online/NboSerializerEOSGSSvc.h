// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/NboSerializer.h"

#include "Online/AuthEOSGS.h"
#include "Online/LobbiesEOSGS.h"
#include "Online/OnlineIdEOSGS.h"
#include "Online/SessionsEOSGS.h"
#include "Online/CoreOnline.h"
#include "Online/NboSerializerCommonSvc.h"

/**
 * Serializes data in network byte order form into a buffer
 */
namespace UE::Online {

namespace NboSerializerEOSGSSvc {

/** NboSerializeToBuffer methods */

inline void SerializeToBuffer(FNboSerializeToBuffer& Ar, const EOnlineServices ServicesType)
{
	Ar << (__underlying_type(EOnlineServices))ServicesType;
}

inline void SerializeToBuffer(FNboSerializeToBuffer& Ar, const FAccountId& UniqueId)
{
	const TArray<uint8> Data = FOnlineAccountIdRegistryEOSGS::GetRegistered(UniqueId.GetOnlineServicesType()).ToReplicationData(UniqueId);

	SerializeToBuffer(Ar, UniqueId.GetOnlineServicesType());

	Ar << Data.Num();
	Ar.WriteBinary(Data.GetData(), Data.Num());
}

inline void SerializeToBuffer(FNboSerializeToBuffer& Ar, const FOnlineSessionId& SessionId)
{
	const TArray<uint8> Data = FOnlineSessionIdRegistryEOSGS::GetRegistered(SessionId.GetOnlineServicesType()).ToReplicationData(SessionId);

	SerializeToBuffer(Ar, SessionId.GetOnlineServicesType());

	Ar << Data.Num();
	Ar.WriteBinary(Data.GetData(), Data.Num());
}

inline void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionMemberIdsSet& SessionMembersSet)
{
	Packet << SessionMembersSet.Num();
	
	for (const FAccountId& SessionMember : SessionMembersSet)
	{
		SerializeToBuffer(Packet, SessionMember);
	}
}

/** NboSerializeFromBuffer methods */

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Ar, EOnlineServices& ServicesType)
{
	using UnderlyingType = __underlying_type(EOnlineServices);
	UnderlyingType Value;
	Ar >> Value;

	ServicesType = (EOnlineServices)Value;
}

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Ar, FAccountId& UniqueId)
{
	EOnlineServices ServicesType;
	SerializeFromBuffer(Ar, ServicesType);

	int32 Size;
	Ar >> Size;

	TArray<uint8> Data;
	Ar.ReadBinaryArray(Data, Size);

	UniqueId = FOnlineAccountIdRegistryEOSGS::GetRegistered(ServicesType).FromReplicationData(Data);
}

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Ar, FOnlineSessionId& SessionId)
{
	EOnlineServices ServicesType;
	SerializeFromBuffer(Ar, ServicesType);

	int32 Size;
	Ar >> Size;

	TArray<uint8> Data;
	Ar.ReadBinaryArray(Data, Size);

	SessionId = FOnlineSessionIdRegistryEOSGS::GetRegistered(ServicesType).FromReplicationData(Data);
}

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionMemberIdsSet& SessionMembersSet)
{
	int32 NumEntries = 0;
	Packet >> NumEntries;

	for (int32 Index = 0; Index < NumEntries; ++Index)
	{
		FAccountId Key;
		SerializeFromBuffer(Packet, Key);

		SessionMembersSet.Emplace(Key);
	}
}

/* NboSerializerNullSvc */ }

/* UE::Online */ }