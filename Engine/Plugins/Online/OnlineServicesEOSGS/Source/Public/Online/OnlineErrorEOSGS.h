// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.6, "Use OnlineErrorEpicCommon.h instead of OnlineErrorEOSGS.h.")

#include "Online/OnlineErrorEpicCommon.h"