// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineIdCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_common.h"

namespace UE::Online {

class IOnlineAccountIdRegistryEOSGS : public IOnlineAccountIdRegistry
{
public:
	virtual FAccountId FindOrAddAccountId(const EOS_ProductUserId ProductUserId) = 0;
	virtual FAccountId FindAccountId(EOS_ProductUserId ProductUserId) const = 0;
	virtual EOS_ProductUserId GetProductUserId(const FAccountId& AccountId) const = 0;
};

/**
 * Account id registry specifically for EOS id's which are segmented.
 */
class FOnlineAccountIdRegistryEOSGS
	: public IOnlineAccountIdRegistryEOSGS
{
public:
	UE_DEPRECATED(5.6, "This method is deprecated, please use the new version taking a EOnlineServices parameter")
	ONLINESERVICESEOSGS_API FOnlineAccountIdRegistryEOSGS();
	ONLINESERVICESEOSGS_API FOnlineAccountIdRegistryEOSGS(EOnlineServices Services);

	virtual ~FOnlineAccountIdRegistryEOSGS() = default;

	// Begin IOnlineAccountIdRegistryEOSGS
	ONLINESERVICESEOSGS_API virtual FAccountId FindOrAddAccountId(const EOS_ProductUserId ProductUserId) override;
	ONLINESERVICESEOSGS_API virtual FAccountId FindAccountId(const EOS_ProductUserId ProductUserId) const override;
	ONLINESERVICESEOSGS_API virtual EOS_ProductUserId GetProductUserId(const FAccountId& AccountId) const override;
	// End IOnlineAccountIdRegistryEOSGS

	// Begin IOnlineAccountIdRegistry
	ONLINESERVICESEOSGS_API virtual FString ToString(const FAccountId& AccountId) const override;
	ONLINESERVICESEOSGS_API virtual FString ToLogString(const FAccountId& AccountId) const override;
	ONLINESERVICESEOSGS_API virtual TArray<uint8> ToReplicationData(const FAccountId& AccountId) const override;
	ONLINESERVICESEOSGS_API virtual FAccountId FromReplicationData(const TArray<uint8>& ReplicationData) override;
	ONLINESERVICESEOSGS_API virtual FAccountId FromStringData(const FString& StringData) override;
	// End IOnlineAccountIdRegistry

	UE_DEPRECATED(5.6, "This method is deprecated, please use the new version taking a EOnlineServices parameter")
	static ONLINESERVICESEOSGS_API IOnlineAccountIdRegistryEOSGS& GetRegistered();

	static ONLINESERVICESEOSGS_API IOnlineAccountIdRegistryEOSGS& GetRegistered(EOnlineServices Services);

private:
	TOnlineBasicAccountIdRegistry<EOS_ProductUserId> Registry;
};

EOS_ProductUserId ONLINESERVICESEOSGS_API GetProductUserId(const FAccountId& AccountId);
EOS_ProductUserId ONLINESERVICESEOSGS_API GetProductUserIdChecked(const FAccountId& AccountId);

UE_DEPRECATED(5.6, "This method is deprecated, please use the new version taking a EOnlineServices parameter")
FAccountId ONLINESERVICESEOSGS_API FindAccountId(const EOS_ProductUserId ProductUserId);
FAccountId ONLINESERVICESEOSGS_API FindAccountId(EOnlineServices Services, const EOS_ProductUserId ProductUserId);

UE_DEPRECATED(5.6, "This method is deprecated, please use the new version taking a EOnlineServices parameter")
FAccountId ONLINESERVICESEOSGS_API FindAccountIdChecked(const EOS_ProductUserId ProductUserId);
FAccountId ONLINESERVICESEOSGS_API FindAccountIdChecked(EOnlineServices Services, const EOS_ProductUserId ProductUserId);

template<typename IdType>
UE_DEPRECATED(5.6, "This method is deprecated, please add an equivalent method or perform the relevant checks in your calling code")
inline bool ValidateOnlineId(const TOnlineId<IdType> OnlineId)
{
	return OnlineId.IsValid() && (OnlineId.GetOnlineServicesType() == EOnlineServices::Epic);
}

} /* namespace UE::Online */
