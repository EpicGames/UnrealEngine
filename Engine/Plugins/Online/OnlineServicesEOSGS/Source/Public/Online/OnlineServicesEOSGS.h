// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineServicesEpicCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_sdk.h"

#if WITH_ENGINE
class FSocketSubsystemEOS;
#endif

using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle>;

namespace UE::Online {

using IPlayerReportsPtr = TSharedPtr<class IPlayerReports>;
using IPlayerSanctionsPtr = TSharedPtr<class IPlayerSanctions>;

class FOnlineServicesEOSGS : public FOnlineServicesEpicCommon
{
public:
	using Super = FOnlineServicesEpicCommon;

	ONLINESERVICESEOSGS_API FOnlineServicesEOSGS(FName InstanceName, FName InstanceConfigName);
	virtual ~FOnlineServicesEOSGS() = default;

	ONLINESERVICESEOSGS_API virtual bool PreInit() override;
	ONLINESERVICESEOSGS_API virtual void Destroy() override;
	ONLINESERVICESEOSGS_API virtual void RegisterComponents() override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FGetResolvedConnectString> GetResolvedConnectString(FGetResolvedConnectString::Params&& Params) override;
	virtual EOnlineServices GetServicesProvider() const override { return EOnlineServices::Epic; }

	ONLINESERVICESEOSGS_API IPlayerReportsPtr GetPlayerReportsInterface();
	ONLINESERVICESEOSGS_API IPlayerSanctionsPtr GetPlayerSanctionsInterface(); 

	static const TCHAR* GetServiceConfigNameStatic() { return TEXT("EOS"); }

protected:

#if WITH_ENGINE
	TSharedPtr<FSocketSubsystemEOS, ESPMode::ThreadSafe> SocketSubsystem;
#endif
};

/* UE::Online */ }
