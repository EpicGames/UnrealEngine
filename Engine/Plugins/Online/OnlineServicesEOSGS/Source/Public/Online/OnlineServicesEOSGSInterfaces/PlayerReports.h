// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

enum class EPlayerReportCategory
{
	Cheating,
	Exploiting,
	OffensiveProfile,
	VerbalAbuse,
	Scamming,
	Spamming,
	Other
};

const TCHAR* LexToString(EPlayerReportCategory Value);
void LexFromString(EPlayerReportCategory& OutValue, const TCHAR* InStr);

struct FSendPlayerReport
{
	static constexpr TCHAR Name[] = TEXT("SendPlayerReport");

	struct Params
	{
		/* Required - Local user performing the operation */
		FAccountId LocalAccountId;
		/* Required - Target user of the player report */
		FAccountId TargetAccountId;
		/* Required - The category of the player */
		EPlayerReportCategory Category;
		/* Optional - Message from the player describing the issue being reported */
		FString Message = "";
		/* Optional - Context information around the issue being reported the game can send. This needs to be in a valid JSON format otherwise the EOS SDK will throw an error */
		FString Context = "";
	};

	struct Result
	{
	};
};

/**
 * Interface definition for the EOS player reports service
 */
class IPlayerReports
{
public:
	/**
	* Send a player report for a player cheating, misconduct, etc... 
	*/
	virtual TOnlineAsyncOpHandle<FSendPlayerReport> SendPlayerReport(FSendPlayerReport::Params&& Params) = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FSendPlayerReport::Params)
	ONLINE_STRUCT_FIELD(FSendPlayerReport::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FSendPlayerReport::Params, TargetAccountId),
	ONLINE_STRUCT_FIELD(FSendPlayerReport::Params, Category), 
	ONLINE_STRUCT_FIELD(FSendPlayerReport::Params, Message), 
	ONLINE_STRUCT_FIELD(FSendPlayerReport::Params, Context)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendPlayerReport::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
