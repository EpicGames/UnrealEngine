// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

enum class EPlayerSanctionAppealReason
{
	IncorrectSanction,
	CompromisedAccount,
	UnfairPunishment,
	AppealForForgiveness
};

const TCHAR* LexToString(EPlayerSanctionAppealReason Value);
void LexFromString(EPlayerSanctionAppealReason& OutValue, const TCHAR* InStr);

struct FCreatePlayerSanctionAppeal
{
	static constexpr TCHAR Name[] = TEXT("CreatePlayerSanctionAppeal");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The reason of the appeal */
		EPlayerSanctionAppealReason Reason;
		/* The sanction id for the sanction that is being appealed */
		FString ReferenceId;
	};

	struct Result
	{
	};
};

struct FActivePlayerSanctionEntry
{
	/* The time the sanction was placed */
	int64_t TimePlaced;
	/* The time the sanction expires */
	int64_t TimeExpires;
	/* The action associated with this sanction */
	FString Action;
	/* The sanction id for the sanction that is being appealed. This needs to be set */
	FString ReferenceId;
};

struct FReadActivePlayerSanctions
{
	static constexpr TCHAR Name[] = TEXT("ReadActivePlayerSanctions");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The account ids of the user we are querying active sanctions for */
		FAccountId TargetAccountId;
	};

	struct Result
	{
		/* The result leaderboard entries */
		TArray<FActivePlayerSanctionEntry> Entries;
	};
};

/**
 * Interface definition for the EOS player reports service
 */
class IPlayerSanctions
{
public:
	/**
	* Send a player sanction appeal
	*/
	virtual TOnlineAsyncOpHandle<FCreatePlayerSanctionAppeal> CreatePlayerSanctionAppeal(FCreatePlayerSanctionAppeal::Params&& Params) = 0;

	/**
	 * Read active player sanction for a specific user
	 */
	virtual TOnlineAsyncOpHandle<FReadActivePlayerSanctions> ReadEntriesForUser(FReadActivePlayerSanctions::Params&& Params) = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FCreatePlayerSanctionAppeal::Params)
	ONLINE_STRUCT_FIELD(FCreatePlayerSanctionAppeal::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCreatePlayerSanctionAppeal::Params, Reason),
	ONLINE_STRUCT_FIELD(FCreatePlayerSanctionAppeal::Params, ReferenceId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadActivePlayerSanctions::Params)
	ONLINE_STRUCT_FIELD(FReadActivePlayerSanctions::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FReadActivePlayerSanctions::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCreatePlayerSanctionAppeal::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadActivePlayerSanctions::Result)
	ONLINE_STRUCT_FIELD(FReadActivePlayerSanctions::Result, Entries)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
