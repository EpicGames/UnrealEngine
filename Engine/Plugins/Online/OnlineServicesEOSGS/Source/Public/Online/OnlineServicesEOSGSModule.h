// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::Online {

class FOnlineServicesEOSGSModule : public IModuleInterface
{
public:
	ONLINESERVICESEOSGS_API static int GetRegistryPriority();

private:
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface
};

} // namespace UE::Online