// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/OnlineComponent.h"
#include "OnlineServicesEOSGSInterfaces/PlayerReports.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_reports_types.h"

enum class EPlayerReportCategory : uint8;
namespace UE::Online { template <typename Result> class TDefaultErrorResultInternal; }
template <typename ResultType> class TFuture;

namespace UE::Online {

class FOnlineServicesEOSGS;

class FPlayerReportsEOSGS : public TOnlineComponent<IPlayerReports>
{
public:
	using Super = TOnlineComponent<IPlayerReports>;

	ONLINESERVICESEOSGS_API FPlayerReportsEOSGS(FOnlineServicesEOSGS& InOwningSubsystem);
	virtual ~FPlayerReportsEOSGS() = default;

	// IOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	ONLINESERVICESEOSGS_API virtual void RegisterCommands() override;

	// IPlayerReports	
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FSendPlayerReport> SendPlayerReport(FSendPlayerReport::Params&& Params) override; 

private:
	EOS_HReports PlayerReportsHandle = nullptr;
};

FString ToLogString(const FSendPlayerReport::Result& PlayerReports);
/* UE::Online */ }


