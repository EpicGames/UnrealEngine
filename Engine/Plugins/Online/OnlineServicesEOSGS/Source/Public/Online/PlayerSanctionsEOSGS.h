// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/OnlineComponent.h"
#include "OnlineServicesEOSGSInterfaces/PlayerSanctions.h"


#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_sanctions_types.h"


namespace UE::Online {

class FOnlineServicesEOSGS;

class FPlayerSanctionsEOSGS : public TOnlineComponent<IPlayerSanctions>
{
public:
	using Super = TOnlineComponent<IPlayerSanctions>;

	ONLINESERVICESEOSGS_API FPlayerSanctionsEOSGS(FOnlineServicesEOSGS& InOwningSubsystem);
	virtual ~FPlayerSanctionsEOSGS() = default;

	// IOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	ONLINESERVICESEOSGS_API virtual void RegisterCommands() override;

	// IPlayerSanctions
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FCreatePlayerSanctionAppeal> CreatePlayerSanctionAppeal(FCreatePlayerSanctionAppeal::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FReadActivePlayerSanctions> ReadEntriesForUser(FReadActivePlayerSanctions::Params&& Params) override;

private:
	EOS_HSanctions PlayerSanctionsHandle = nullptr;
};

FString ToLogString(const FReadActivePlayerSanctions::Result& ReadPlayerSanctionResult);
FString ToLogString(const FReadActivePlayerSanctions::Result& ReadPlayerSanctionResult);

/* UE::Online */ }


