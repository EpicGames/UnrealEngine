// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsLAN.h"

#include "Online/OnlineServicesEOSGSTypes.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_sessions_types.h"
#include "eos_ui_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;
class FSessionsEOSGS;

class FOnlineSessionIdRegistryEOSGS : public FOnlineSessionIdRegistryLAN
{
public:
	UE_DEPRECATED(5.6, "This method is deprecated, please use GetRegistered(EOnlineServices)")
	static FOnlineSessionIdRegistryEOSGS& Get() { return GetRegistered(EOnlineServices::Epic); }
	
	static ONLINESERVICESEOSGS_API FOnlineSessionIdRegistryEOSGS& GetRegistered(EOnlineServices Services);
	
private:
	friend class FOnlineServicesEOSGSModule;
	friend class FOnlineServicesEpicGameModule;
	ONLINESERVICESEOSGS_API FOnlineSessionIdRegistryEOSGS(EOnlineServices ServicesType);
};

class FOnlineSessionInviteIdRegistryEOSGS : public FOnlineSessionInviteIdStringRegistry
{
public:
	UE_DEPRECATED(5.6, "This method is deprecated, please use GetRegistered(EOnlineServices)")
	static FOnlineSessionInviteIdRegistryEOSGS& Get() { return GetRegistered(EOnlineServices::Epic); }

	static ONLINESERVICESEOSGS_API FOnlineSessionInviteIdRegistryEOSGS& GetRegistered(EOnlineServices Services);

private:
	friend class FOnlineServicesEOSGSModule;
	friend class FOnlineServicesEpicGameModule;
	ONLINESERVICESEOSGS_API FOnlineSessionInviteIdRegistryEOSGS(EOnlineServices ServicesType);
};

struct FSessionModificationHandleEOSGS : FNoncopyable
{
	EOS_HSessionModification ModificationHandle;

	FSessionModificationHandleEOSGS(EOS_HSessionModification InModificationHandle)
		: ModificationHandle(InModificationHandle)
	{
	}

	~FSessionModificationHandleEOSGS()
	{
		EOS_SessionModification_Release(ModificationHandle);
	}
};

struct FSessionSearchHandleEOSGS : FNoncopyable
{
	EOS_HSessionSearch SearchHandle;

	FSessionSearchHandleEOSGS(EOS_HSessionSearch InSearchHandle)
		: SearchHandle(InSearchHandle)
	{
	}

	~FSessionSearchHandleEOSGS()
	{
		EOS_SessionSearch_Release(SearchHandle);
	}
};

struct FSessionDetailsHandleEOSGS : FNoncopyable
{
	EOS_HSessionDetails SessionDetailsHandle;

	FSessionDetailsHandleEOSGS(EOS_HSessionDetails InSessionDetailsHandle)
		: SessionDetailsHandle(InSessionDetailsHandle)
	{
	}

	~FSessionDetailsHandleEOSGS()
	{
		EOS_SessionDetails_Release(SessionDetailsHandle);
	}
};

class FSessionEOSGS : public FSessionLAN
{
public:
	FSessionEOSGS() = default;
	FSessionEOSGS(const FSessionEOSGS& InSession) = default;

	/** This constructor should only be used by BuildSessionFromDetailsHandle, after all user ids in the session have been resolved. */
	FSessionEOSGS(const FSessionsEOSGS& SessionsInstance, const TSharedPtr<FSessionDetailsHandleEOSGS>& InSessionDetailsHandle);

	static const FSessionEOSGS& Cast(const ISession& InSession);

public:
	/** Session details handle */
	TSharedPtr<FSessionDetailsHandleEOSGS> SessionDetailsHandle;
};

struct FUpdateSessionJoinabilityImpl
{
	static constexpr TCHAR Name[] = TEXT("UpdateSessionJoinabilityImpl");

	struct Params
	{
		/** Name for the session, needed to start or end it */
		FName SessionName;

		/** Whether players are accepted as new members in the session. */
		bool bAllowNewMembers;
	};

	struct Result
	{
	};
};

typedef FUpdateSessionJoinabilityImpl::Params FUpdateSessionJoinabilityParams;

struct FUpdateSessionImplEOSGS
{
	static constexpr TCHAR Name[] = TEXT("UpdateSessionImplEOSGS");

	struct Params
	{
		/** Handle for the session modification operation */
		TSharedRef<FSessionModificationHandleEOSGS> SessionModificationHandle;

		/** If set, it will use the values set in the struct to update the session's joinability */
		TOptional<FUpdateSessionJoinabilityParams> UpdateJoinabilitySettings;
	};

	struct Result
	{
		/** EOSGS Session Id for the created or modified session */
		FString NewSessionId;
	};
};

struct FSendSingleSessionInviteImpl
{
	static constexpr TCHAR Name[] = TEXT("SendSingleSessionInviteImpl");

	struct Params
	{
		FAccountId LocalAccountId;

		FName SessionName;

		FAccountId TargetAccountId;
	};

	struct Result
	{
	};
};

struct FBuildSessionFromDetailsHandle
{
	static constexpr TCHAR Name[] = TEXT("BuildSessionFromDetailsHandle");

	struct Params
	{
		/** User which will drive the id resolution */
		FAccountId LocalAccountId;

		/** EOS session details handle used to extract the data */
		TSharedRef<FSessionDetailsHandleEOSGS> SessionDetailsHandleEOSGS;
	};

	struct Result
	{
		/** User which started the resolution operation */
		FAccountId LocalAccountId;

		/** Session built from the details handle */
		TSharedRef<FSessionCommon> Session;
	};
};

class FSessionsEOSGS : public FSessionsLAN
{
public:
	friend class FSessionEOSGS;

	using Super = FSessionsLAN;

	ONLINESERVICESEOSGS_API FSessionsEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);
	virtual ~FSessionsEOSGS() = default;

	// TOnlineComponent
	ONLINESERVICESEOSGS_API void Initialize() override;
	ONLINESERVICESEOSGS_API void Shutdown() override;

	// FSessionsCommon
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FSetPresenceSession> SetPresenceSession(FSetPresenceSession::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FClearPresenceSession> ClearPresenceSession(FClearPresenceSession::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FCreateSession>> CreateSessionImpl(const FCreateSession::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FUpdateSessionSettings>> UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FLeaveSession>> LeaveSessionImpl(const FLeaveSession::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FFindSessions>> FindSessionsImpl(const FFindSessions::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FJoinSession>> JoinSessionImpl(const FJoinSession::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FAddSessionMember>> AddSessionMemberImpl(const FAddSessionMember::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FRemoveSessionMember>> RemoveSessionMemberImpl(const FRemoveSessionMember::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FSendSessionInvite>> SendSessionInviteImpl(const FSendSessionInvite::Params& Params) override;
	ONLINESERVICESEOSGS_API virtual TFuture<TOnlineResult<FRejectSessionInvite>> RejectSessionInviteImpl(const FRejectSessionInvite::Params& Params) override;

protected:
	ONLINESERVICESEOSGS_API virtual TOptional<FOnlineError> CheckState(const FCreateSession::Params& Params) const override;
	ONLINESERVICESEOSGS_API virtual TOptional<FOnlineError> CheckState(const FJoinSession::Params& Params) const override;

	ONLINESERVICESEOSGS_API void RegisterEventHandlers();
	ONLINESERVICESEOSGS_API void UnregisterEventHandlers();
	ONLINESERVICESEOSGS_API void HandleSessionInviteReceived(const EOS_Sessions_SessionInviteReceivedCallbackInfo* Data);
	ONLINESERVICESEOSGS_API void HandleSessionInviteAccepted(const EOS_Sessions_SessionInviteAcceptedCallbackInfo* Data);
	ONLINESERVICESEOSGS_API void HandleJoinSessionAccepted(const EOS_Sessions_JoinSessionAcceptedCallbackInfo* Data);

	ONLINESERVICESEOSGS_API void SetHostAddress(EOS_HSessionModification& SessionModHAndle, FString HostAddress);
	ONLINESERVICESEOSGS_API void SetJoinInProgressAllowed(EOS_HSessionModification& SessionModHandle, bool bIsJoinInProgressAllowed);
	ONLINESERVICESEOSGS_API void SetInvitesAllowed(EOS_HSessionModification& SessionModHandle, bool bAreInvitesAllowed);
	ONLINESERVICESEOSGS_API void SetPermissionLevel(EOS_HSessionModification& SessionModHandle, const ESessionJoinPolicy& JoinPolicy);
	ONLINESERVICESEOSGS_API void SetBucketId(EOS_HSessionModification& SessionModHandle, const FString& NewBucketId);
	ONLINESERVICESEOSGS_API void SetMaxPlayers(EOS_HSessionModification& SessionModHandle, const uint32& NewMaxPlayers);
	ONLINESERVICESEOSGS_API void AddAttribute(EOS_HSessionModification& SessionModificationHandle, const FSchemaAttributeId& Key, const FCustomSessionSetting& Value);
	ONLINESERVICESEOSGS_API void RemoveAttribute(EOS_HSessionModification& SessionModificationHandle, const FSchemaAttributeId& Key);

	ONLINESERVICESEOSGS_API void SetSessionSearchMaxResults(FSessionSearchHandleEOSGS& SessionSearchHandle, uint32 MaxResults);
	ONLINESERVICESEOSGS_API void SetSessionSearchParameters(FSessionSearchHandleEOSGS& SessionSearchHandle, TArray<FFindSessionsSearchFilter> Filters);
	ONLINESERVICESEOSGS_API void SetSessionSearchSessionId(FSessionSearchHandleEOSGS& SessionSearchHandle, const FOnlineSessionId& SessionId);
	ONLINESERVICESEOSGS_API void SetSessionSearchTargetId(FSessionSearchHandleEOSGS& SessionSearchHandle, const FAccountId& TargetAccountId);

	/**
	 * Writes all values in the passed SessionSettings to the SessionModificationHandle
	 */
	ONLINESERVICESEOSGS_API void WriteCreateSessionModificationHandle(EOS_HSessionModification& SessionModificationHandle, const FCreateSession::Params& Params);

	/**
	 * Writes only the new values for all updated session settings to the SessionModificationHandle
	 */
	ONLINESERVICESEOSGS_API void WriteUpdateSessionModificationHandle(EOS_HSessionModification& SessionModificationHandle, const FSessionSettingsUpdate& NewSettings);

	/**
	 * Writes all relevant values set in the FindSessions parameters into the SessionSearchHandle
	 */
	ONLINESERVICESEOSGS_API void WriteSessionSearchHandle(FSessionSearchHandleEOSGS& SessionSearchHandle, const FFindSessions::Params& Params);

	/**
	 * Internal method used by both CreateSession and UpdateSession to process a session update at the API level
	 */
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FUpdateSessionImplEOSGS>> UpdateSessionImplEOSGS(FUpdateSessionImplEOSGS::Params&& Params);

	/**
	 * Internal method called after UpdateSessionImpl to update joinability options for a session
	 */
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FUpdateSessionJoinabilityImpl>> UpdateSessionJoinabilityImpl(FUpdateSessionJoinabilityImpl::Params&& Params);

	/**
	 * Internal method called by SendSessionInvite for every user
	 */
	ONLINESERVICESEOSGS_API TFuture<TDefaultErrorResult<FSendSingleSessionInviteImpl>> SendSingleSessionInviteImpl(FSendSingleSessionInviteImpl::Params&& Params);

	ONLINESERVICESEOSGS_API FOnlineSessionId CreateSessionId(const FString& SessionId) const;
	ONLINESERVICESEOSGS_API FSessionInviteId CreateSessionInviteId(const FString& SessionInviteId) const;

	/**
	 * Builds a session from an EOS Session Details Handle. Asynchronous due to the id resolution process
	 */
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FBuildSessionFromDetailsHandle> BuildSessionFromDetailsHandle(FBuildSessionFromDetailsHandle::Params&& Params);

	/**
	 * Builds a session from an invite id, calling BuildSessionFromDetailsHandle
	 */
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FBuildSessionFromDetailsHandle> BuildSessionFromInvite(const FAccountId& LocalAccountId, const FString& InInviteId);

	/**
	 * Builds a session from a UI event id, calling BuildSessionFromDetailsHandle
	 */
	ONLINESERVICESEOSGS_API TOnlineAsyncOpHandle<FBuildSessionFromDetailsHandle> BuildSessionFromUIEvent(const FAccountId& LocalAccountId, const EOS_UI_EventId& UIEventId);

private:
	// FSessionsLAN
	ONLINESERVICESEOSGS_API virtual void AppendSessionToPacket(FNboSerializeToBuffer& Packet, const FSessionLAN& Session) override;
	ONLINESERVICESEOSGS_API virtual void ReadSessionFromPacket(FNboSerializeFromBuffer& Packet, FSessionLAN& Session) override;

	ONLINESERVICESEOSGS_API FAccountId FindAccountId(const EOS_ProductUserId ProductUserId);

protected:
	EOS_HSessions SessionsHandle = nullptr;

	FEOSEventRegistrationPtr OnSessionInviteReceivedEventRegistration;
	FEOSEventRegistrationPtr OnSessionInviteAcceptedEventRegistration;
	FEOSEventRegistrationPtr OnJoinSessionAcceptedEventRegistration;

	TMap<FAccountId, TSharedRef<FSessionSearchHandleEOSGS>> CurrentSessionSearchHandleEOSGSUserMap;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FUpdateSessionJoinabilityImpl::Params)
	ONLINE_STRUCT_FIELD(FUpdateSessionJoinabilityImpl::Params, SessionName),
	ONLINE_STRUCT_FIELD(FUpdateSessionJoinabilityImpl::Params, bAllowNewMembers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdateSessionJoinabilityImpl::Result)
END_ONLINE_STRUCT_META()		
		
BEGIN_ONLINE_STRUCT_META(FUpdateSessionImplEOSGS::Params)
	ONLINE_STRUCT_FIELD(FUpdateSessionImplEOSGS::Params, SessionModificationHandle)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdateSessionImplEOSGS::Result)
	ONLINE_STRUCT_FIELD(FUpdateSessionImplEOSGS::Result, NewSessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendSingleSessionInviteImpl::Params)
	ONLINE_STRUCT_FIELD(FSendSingleSessionInviteImpl::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FSendSingleSessionInviteImpl::Params, SessionName),
	ONLINE_STRUCT_FIELD(FSendSingleSessionInviteImpl::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendSingleSessionInviteImpl::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBuildSessionFromDetailsHandle::Params)
	ONLINE_STRUCT_FIELD(FBuildSessionFromDetailsHandle::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FBuildSessionFromDetailsHandle::Params, SessionDetailsHandleEOSGS)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBuildSessionFromDetailsHandle::Result)
	ONLINE_STRUCT_FIELD(FBuildSessionFromDetailsHandle::Result, LocalAccountId),
	ONLINE_STRUCT_FIELD(FBuildSessionFromDetailsHandle::Result, Session)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
