// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Sessions.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_sessions_types.h"

namespace UE::Online {

static FName EOSGS_ALLOW_NEW_MEMBERS_ATTRIBUTE_KEY = TEXT("EOSGS_ALLOW_NEW_MEMBERS");
static FName EOSGS_ANTI_CHEAT_PROTECTED_ATTRIBUTE_KEY = TEXT("EOSGS_ANTI_CHEAT_PROTECTED_ATTRIBUTE_KEY");
static FName EOSGS_IS_DEDICATED_SERVER_SESSION_ATTRIBUTE_KEY = TEXT("EOSGS_IS_DEDICATED_SERVER_SESSION_ATTRIBUTE_KEY");
static FName EOSGS_SCHEMA_NAME_ATTRIBUTE_KEY = TEXT("EOSGS_SCHEMA_NAME_ATTRIBUTE_KEY");

static FName EOSGS_BUCKET_ID_ATTRIBUTE_KEY = TEXT("EOSGS_BUCKET_ID_ATTRIBUTE_KEY");
static FName EOSGS_HOST_ADDRESS_ATTRIBUTE_KEY = TEXT("EOSGS_HOST_ADDRESS_ATTRIBUTE_KEY");

EOS_EOnlineSessionPermissionLevel ToServiceType(const ESessionJoinPolicy& Value);
ESessionJoinPolicy FromServiceType(const EOS_EOnlineSessionPermissionLevel& Value);

EOS_ESessionAttributeAdvertisementType ToServiceType(const ESchemaAttributeVisibility& Value);
ESchemaAttributeVisibility FromServiceType(const EOS_ESessionAttributeAdvertisementType& Value);

EOS_EOnlineComparisonOp ToServiceType(const ESchemaAttributeComparisonOp& Value);

enum class ESessionAttributeConversionType
{
	ToService,
	FromService
};

template <ESessionAttributeConversionType>
class FSessionAttributeConverter
{
public:
};

template<>
class FSessionAttributeConverter<ESessionAttributeConversionType::ToService>
{
public:
	FSessionAttributeConverter(const FSchemaAttributeId& Key, const FSchemaVariant& Value);

	FSessionAttributeConverter(const TPair<FSchemaAttributeId, FSchemaVariant>& InData);

	const EOS_Sessions_AttributeData& GetAttributeData() const { return AttributeData; }

private:
	FTCHARToUTF8 KeyConverterStorage;
	TOptional<FTCHARToUTF8> ValueConverterStorage; // TOptional because we'll only use it if the FSessionVariant type is FString
	EOS_Sessions_AttributeData AttributeData;
};

template<>
class FSessionAttributeConverter<ESessionAttributeConversionType::FromService>
{
public:
	FSessionAttributeConverter(const EOS_Sessions_AttributeData& InData);

	const TPair<FSchemaAttributeId, FSchemaVariant>& GetAttributeData() const { return AttributeData; }

private:
	TPair<FSchemaAttributeId, FSchemaVariant> AttributeData;
};

/* UE::Online */ }