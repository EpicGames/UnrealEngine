// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/StatsCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_stats_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;

/**
 * Because the value type is int32 in EOS, stats types are converted to int32. So these are limitations: 
 *	- String type of Stats is not supported;
 *	- double type of Stats is casted to int32, and precision is UE_ONLINE_STAT_EOS_DOUBLE_PRECISION. Out of range value will be clamped;
 *	- int64 type of Stats is casted to int32. Out of range value will be clamped;
 */
class FStatsEOSGS : public FStatsCommon
{
public:
	using Super = FStatsCommon;

	ONLINESERVICESEOSGS_API FStatsEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);

	// TOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;

	// IStats
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUpdateStats> UpdateStats(FUpdateStats::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FQueryStats> QueryStats(FQueryStats::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FBatchQueryStats> BatchQueryStats(FBatchQueryStats::Params&& Params) override;

protected:
	ONLINESERVICESEOSGS_API void ReadStatsFromEOSResult(const EOS_Stats_OnQueryStatsCompleteCallbackInfo* Data, const TArray<FString>& StatNames, TMap<FString, FStatValue>& OutStats);

	EOS_HStats StatsHandle = nullptr;
	TArray<FUserStats> BatchQueriedUsersStats;
};

/* UE::Online */ }
