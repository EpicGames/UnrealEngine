// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/TitleFileCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_titlestorage_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;

struct FTitleFileEOSGSConfig
{
	FString SearchTag;
	int32 ReadChunkLengthBytes = 4096;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FTitleFileEOSGSConfig)
	ONLINE_STRUCT_FIELD(FTitleFileEOSGSConfig, SearchTag),
	ONLINE_STRUCT_FIELD(FTitleFileEOSGSConfig, ReadChunkLengthBytes)
END_ONLINE_STRUCT_META()

/* Meta */ }

class FTitleFileEOSGS : public FTitleFileCommon
{
public:
	using Super = FTitleFileCommon;

	ONLINESERVICESEOSGS_API FTitleFileEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);
	virtual ~FTitleFileEOSGS() = default;

	// IOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	ONLINESERVICESEOSGS_API virtual void UpdateConfig() override;

	// ITitleFile
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FTitleFileGetEnumeratedFiles> GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FTitleFileReadFile> ReadFile(FTitleFileReadFile::Params&& Params) override;

protected:
	EOS_HTitleStorage TitleStorageHandle = nullptr;

	FTitleFileEOSGSConfig Config;

	TOptional<TArray<FString>> EnumeratedFiles;

	static ONLINESERVICESEOSGS_API EOS_TitleStorage_EReadResult EOS_CALL OnReadFileDataStatic(const EOS_TitleStorage_ReadFileDataCallbackInfo* Data);
	static ONLINESERVICESEOSGS_API void EOS_CALL OnFileTransferProgressStatic(const EOS_TitleStorage_FileTransferProgressCallbackInfo* Data);
	static ONLINESERVICESEOSGS_API void EOS_CALL OnReadFileCompleteStatic(const EOS_TitleStorage_ReadFileCallbackInfo* Data);
};

/* UE::Online */ }
