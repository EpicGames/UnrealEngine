// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/UserFileCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_playerdatastorage_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;

struct FUserFileEOSGSConfig
{
	int32 ChunkLengthBytes = 4096;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FUserFileEOSGSConfig)
	ONLINE_STRUCT_FIELD(FUserFileEOSGSConfig, ChunkLengthBytes)
END_ONLINE_STRUCT_META()

/* Meta */ }

class FUserFileEOSGS : public FUserFileCommon
{
public:
	using Super = FUserFileCommon;

	ONLINESERVICESEOSGS_API FUserFileEOSGS(FOnlineServicesEpicCommon& InOwningSubsystem);
	virtual ~FUserFileEOSGS() = default;

	// IOnlineComponent
	ONLINESERVICESEOSGS_API virtual void Initialize() override;
	ONLINESERVICESEOSGS_API virtual void UpdateConfig() override;

	// IUserFile
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUserFileEnumerateFiles> EnumerateFiles(FUserFileEnumerateFiles::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineResult<FUserFileGetEnumeratedFiles> GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUserFileReadFile> ReadFile(FUserFileReadFile::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUserFileWriteFile> WriteFile(FUserFileWriteFile::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUserFileCopyFile> CopyFile(FUserFileCopyFile::Params&& Params) override;
	ONLINESERVICESEOSGS_API virtual TOnlineAsyncOpHandle<FUserFileDeleteFile> DeleteFile(FUserFileDeleteFile::Params&& Params) override;

protected:
	EOS_HPlayerDataStorage PlayerDataStorageHandle = nullptr;

	FUserFileEOSGSConfig Config;

	TMap<FAccountId, TArray<FString>> UserToFiles;

	static ONLINESERVICESEOSGS_API EOS_PlayerDataStorage_EReadResult EOS_CALL OnReadFileDataStatic(const EOS_PlayerDataStorage_ReadFileDataCallbackInfo* Data);
	static ONLINESERVICESEOSGS_API void EOS_CALL OnReadFileCompleteStatic(const EOS_PlayerDataStorage_ReadFileCallbackInfo* Data);

	static ONLINESERVICESEOSGS_API EOS_PlayerDataStorage_EWriteResult EOS_CALL OnWriteFileDataStatic(const EOS_PlayerDataStorage_WriteFileDataCallbackInfo* Data, void* OutDataBuffer, uint32_t* OutDataWritten);
	static ONLINESERVICESEOSGS_API void EOS_CALL OnWriteFileCompleteStatic(const EOS_PlayerDataStorage_WriteFileCallbackInfo* Data);

	static ONLINESERVICESEOSGS_API void EOS_CALL OnFileTransferProgressStatic(const EOS_PlayerDataStorage_FileTransferProgressCallbackInfo* Data);
};

/* UE::Online */ }
